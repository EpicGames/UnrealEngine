{
	"FileVersion": 3,
	"Version": 1,
	"VersionName": "1.0",
	"FriendlyName": "SQLite",
	"Description": "Provides a lightweight C++ wrapper for creating and manipulating SQLite databases. It uses the sqlite C library please refer to Engine/Source/ThirdParty/Licenses/SQLite_v3.47.1.license for license details.",
	"Category": "Database",
	"CreatedBy": "Epic Games, Inc.",
	"CreatedByURL": "https://epicgames.com",
	"DocsURL": "",
	"MarketplaceURL": "",
	"SupportURL": "",
	"EnabledByDefault": false,
	"CanContainContent": false,
	"IsBetaVersion": false,
	"Installed": false,
	"SupportedPrograms": [ "UnrealMultiUserServer", "CoopMultiUserServer", "UnrealMultiUserSlateServer", "UnrealRecoverySvc", "LiveLinkHub" ],
	"Modules": [
		{
			"Name": "SQLiteCore",
			"Type": "Runtime",
			"LoadingPhase": "PreDefault"
		}
	]
}
