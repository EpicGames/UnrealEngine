// Copyright Epic Games, Inc. All Rights Reserved.

#include "SQLiteCore.h"
#include "IncludeSQLite.h"

IMPLEMENT_MODULE(FSQLiteCore, SQLiteCore)

void FSQLiteCore::StartupModule()
{
	StaticInitializeSQLite();
}

void FSQLiteCore::ShutdownModule()
{
	StaticShutdownSQLite();
}

void FSQLiteCore::StaticInitializeSQLite()
{
	if (!bInitializedSQLite)
	{
		// SQLite using a custom platform needs to perform some additional config before calling init
#if SQLITE_OS_OTHER
		extern int sqlite3_ue_config();
		if (sqlite3_ue_config() != SQLITE_OK)
		{
			return;
		}
#endif

		bInitializedSQLite = sqlite3_initialize() == SQLITE_OK;
	}
}

void FSQLiteCore::StaticShutdownSQLite()
{
	if (bInitializedSQLite)
	{
		sqlite3_shutdown();
	}
}

bool FSQLiteCore::bInitializedSQLite = false;
