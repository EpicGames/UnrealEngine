// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Modules/ModuleManager.h"

class FSQLiteCore : public IModuleInterface
{
public:
	/** IModuleInterface implementation */
	virtual void StartupModule() override;
	virtual void ShutdownModule() override;

	/** Initialize SQLite (if it hasn't been already) */
	static void StaticInitializeSQLite();

	/** Shutdown SQLite (if it was initialized) */
	static void StaticShutdownSQLite();

private:
	/** True if SQLite has been initialized */
	static bool bInitializedSQLite;
};
