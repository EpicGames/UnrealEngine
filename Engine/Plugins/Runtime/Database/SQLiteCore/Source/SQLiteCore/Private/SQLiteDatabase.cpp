// Copyright Epic Games, Inc. All Rights Reserved.

#include "SQLiteDatabase.h"
#include "SQLiteCore.h"
#include "IncludeSQLite.h"

#include "Misc/Paths.h"
#include "HAL/FileManager.h"
#include "Misc/AssertionMacros.h"
#include "Containers/StringConv.h"
#include "Trace/Trace.h"
#include "Trace/Trace.inl"

DEFINE_LOG_CATEGORY_STATIC(LogSQLiteDatabase, Log, All);

FSQLiteDatabase::FSQLiteDatabase()
	: Database(nullptr)
{
	// Ensure SQLite is initialized (as our module may not have loaded yet)
	FSQLiteCore::StaticInitializeSQLite();
}

FSQLiteDatabase::~FSQLiteDatabase()
{
	checkf(!Database, TEXT("Destructor called while an SQLite database was still open. Did you forget to call Close?"));
}

FSQLiteDatabase::FSQLiteDatabase(FSQLiteDatabase&& Other)
	: Database(Other.Database)
{
	Other.Database = nullptr;
}

FSQLiteDatabase& FSQLiteDatabase::operator=(FSQLiteDatabase&& Other)
{
	if (this != &Other)
	{
		Close();

		Database = Other.Database;
		Other.Database = nullptr;
	}
	return *this;
}

bool FSQLiteDatabase::IsValid() const
{
	return Database != nullptr;
}

bool FSQLiteDatabase::Open(const TCHAR* InFilename, const ESQLiteDatabaseOpenMode InOpenMode)
{
	if (Database)
	{
		return false;
	}
	
	OriginalPath = InFilename;
	ShortName = FPaths::GetBaseFilename(InFilename);

	int32 OpenFlags = 0;
	switch (InOpenMode)
	{
	case ESQLiteDatabaseOpenMode::ReadOnly:
		OpenFlags = SQLITE_OPEN_READONLY;
		break;
	case ESQLiteDatabaseOpenMode::ReadWrite:
		OpenFlags = SQLITE_OPEN_READWRITE;
		break;
	case ESQLiteDatabaseOpenMode::ReadWriteCreate:
		OpenFlags = SQLITE_OPEN_READWRITE | SQLITE_OPEN_CREATE;
		break;
	}
	checkf(OpenFlags != 0, TEXT("SQLite open flags were zero! Unhandled ESQLiteDatabaseOpenMode?"));

	if (OpenFlags & SQLITE_OPEN_CREATE)
	{
		// Try to ensure that the outer directory exists if we're allowed to create the database file (SQLite won't do this for you)
		IFileManager::Get().MakeDirectory(*FPaths::GetPath(InFilename), true);
	}

	if (sqlite3_open_v2(TCHAR_TO_UTF8(InFilename), &Database, OpenFlags, nullptr) != SQLITE_OK)
	{
		if (Database)
		{
			UE_LOGF(LogSQLiteDatabase, Warning, "Failed to open database '%ls': %ls", InFilename, *GetLastError());

			// Partially opened a database - try and close it
			sqlite3_close(Database);
			Database = nullptr;
		}
		return false;
	}
	UE_LOGF(LogSQLiteDatabase, Log, "Opened database '%ls'", InFilename);

	return true;
}

bool FSQLiteDatabase::Close()
{
	if (!Database)
	{
		return false;
	}

	const int32 Result = sqlite3_close(Database);
	if (Result != SQLITE_OK)
	{
		return false;
	}
	
	Database = nullptr;
	return true;
}

FString FSQLiteDatabase::GetFilename() const
{
	const char* FilenameStr = Database ? sqlite3_db_filename(Database, "main") : nullptr;
	if (FilenameStr)
	{
		return UTF8_TO_TCHAR(FilenameStr);
	}
	return FString();
}

bool FSQLiteDatabase::GetApplicationId(int32& OutApplicationId) const
{
	return const_cast<FSQLiteDatabase*>(this)->Execute(TEXT("PRAGMA application_id;"), [&OutApplicationId](const FSQLitePreparedStatement& InStatement)
	{
		InStatement.GetColumnValueByIndex(0, OutApplicationId);
		return ESQLitePreparedStatementExecuteRowResult::Stop;
	}) == 1;
}

bool FSQLiteDatabase::SetApplicationId(const int32 InApplicationId)
{
	return Execute(*FString::Printf(TEXT("PRAGMA application_id = %d;"), InApplicationId));
}

bool FSQLiteDatabase::GetUserVersion(int32& OutUserVersion) const
{
	return const_cast<FSQLiteDatabase*>(this)->Execute(TEXT("PRAGMA user_version;"), [&OutUserVersion](const FSQLitePreparedStatement& InStatement)
	{
		InStatement.GetColumnValueByIndex(0, OutUserVersion);
		return ESQLitePreparedStatementExecuteRowResult::Stop;
	}) == 1;
}

bool FSQLiteDatabase::SetUserVersion(const int32 InUserVersion)
{
	return Execute(*FString::Printf(TEXT("PRAGMA user_version = %d;"), InUserVersion));
}

bool FSQLiteDatabase::Execute(const TCHAR* InStatement)
{
	if (!Database)
	{
		return false;
	}

	// Create a prepared statement
	FSQLitePreparedStatement Statement(*this, InStatement);
	if (!Statement.IsValid())
	{
		return false;
	}

	// Execute it
	return Statement.Execute();
}

int64 FSQLiteDatabase::Execute(const TCHAR* InStatement, TFunctionRef<ESQLitePreparedStatementExecuteRowResult(const FSQLitePreparedStatement&)> InCallback)
{
	if (!Database)
	{
		return false;
	}

	// Create a prepared statement
	FSQLitePreparedStatement Statement(*this, InStatement);
	if (!Statement.IsValid())
	{
		return false;
	}

	// Execute it
	return Statement.Execute(InCallback);
}

FSQLitePreparedStatement FSQLiteDatabase::PrepareStatement(const TCHAR* InStatement, const ESQLitePreparedStatementFlags InFlags)
{
	return Database
		? FSQLitePreparedStatement(*this, InStatement, InFlags)
		: FSQLitePreparedStatement();
}

FString FSQLiteDatabase::GetErrorForCode(const int32 ResultCode)
{
	if (const char* ErrorStr = sqlite3_errstr(ResultCode))
	{
		return UTF8_TO_TCHAR(ErrorStr);
	}
	return FString();
}

FString FSQLiteDatabase::GetLastError() const
{
	const char* ErrorStr = Database ? sqlite3_errmsg(Database) : nullptr;
	if (ErrorStr)
	{
		return UTF8_TO_TCHAR(ErrorStr);
	}
	return FString();
}

int64 FSQLiteDatabase::GetLastInsertRowId() const
{
	return Database
		? sqlite3_last_insert_rowid(Database)
		: 0;
}

UE_TRACE_EVENT_BEGIN(Cpu, Sqlite_PerformQuickIntegrityCheck, NoSync)
	UE_TRACE_EVENT_FIELD(UE::Trace::WideString, Path)
UE_TRACE_EVENT_END()

bool FSQLiteDatabase::PerformQuickIntegrityCheck() const
{
#if CPUPROFILERTRACE_ENABLED
	UE_TRACE_LOG_SCOPED_T(Cpu, Sqlite_PerformQuickIntegrityCheck, CpuChannel)
		<< Sqlite_PerformQuickIntegrityCheck.Path(*OriginalPath);
#endif // CPUPROFILERTRACE_ENABLED
	bool OutIntegrityOk = true;
	bool bSuccessful = const_cast<FSQLiteDatabase*>(this)->Execute(TEXT("pragma quick_check;"), [&OutIntegrityOk](const FSQLitePreparedStatement& InStatement)
	{
		FString Results;
		InStatement.GetColumnValueByIndex(0, Results);
		OutIntegrityOk = Results == TEXT("ok");

		return ESQLitePreparedStatementExecuteRowResult::Stop;
	}) == 1;

	return bSuccessful && OutIntegrityOk;
}
