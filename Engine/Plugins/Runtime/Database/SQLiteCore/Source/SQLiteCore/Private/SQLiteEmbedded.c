// Copyright Epic Games, Inc. All Rights Reserved.

/** Needed to define USING_CODE_ANALYSIS, and allow direct access to the SQLite API from outside this module */
#if defined(PLATFORM_MICROSOFT) && PLATFORM_MICROSOFT
 #if defined(PLATFORM_WINDOWS) && PLATFORM_WINDOWS
 #include "Windows/WindowsPlatformCodeAnalysis.h"
 #endif
#define DLLEXPORT __declspec(dllexport)
#define DLLIMPORT __declspec(dllimport)
#elif defined(__clang__)
#include "Clang/ClangPlatformCodeAnalysis.h"
#define DLLEXPORT __attribute__((visibility("default")))
#define DLLIMPORT __attribute__((visibility("default")))
#endif

/** Include SQLite, but not if we're building for analysis as the code emits warnings */
#if !defined(USING_CODE_ANALYSIS) || !USING_CODE_ANALYSIS
#include "HAL/PreprocessorHelpers.h"
#include COMPILED_PLATFORM_HEADER(PlatformCompilerPreSetup.h)
UE_COMPILER_THIRD_PARTY_INCLUDES_START
#include "sqlite/sqlite3.inl"
UE_COMPILER_THIRD_PARTY_INCLUDES_END
#endif
