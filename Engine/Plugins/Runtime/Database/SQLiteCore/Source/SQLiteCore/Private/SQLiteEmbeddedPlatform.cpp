// Copyright Epic Games, Inc. All Rights Reserved.

#if SQLITE_OS_OTHER

#include "CoreTypes.h"
#include "Misc/Paths.h"
#include "Misc/ScopeLock.h"
#include "Math/RandomStream.h"
#include "HAL/PlatformProcess.h"
#include "HAL/PlatformFileManager.h"
#include "HAL/PlatformFile.h"
#include "Templates/Atomic.h"

THIRD_PARTY_INCLUDES_START
#include "sqlite/sqlite3.h"
THIRD_PARTY_INCLUDES_END

/** Malloc functions used by SQLite (see sqlite3_mem_methods) */
struct FSQLiteMallocFuncs
{
public:
	/** Register the malloc system */
	static void Register()
	{
		static const sqlite3_mem_methods MallocFuncs = {
			&Malloc,
			&Free,
			&Realloc,
			&Size,
			&Roundup,
			&Init,
			&Shutdown,
			nullptr,
		};
		static const sqlite3_mem_methods WorkaroundMallocFuncs = {
			&WorkaroundMalloc,
			&WorkaroundFree,
			&WorkaroundRealloc,
			&WorkaroundSize,
			&WorkaroundRoundup,
			&Init,
			&Shutdown,
			nullptr,
		};

		// SQLite needs a working FMemory::GetAllocSize, so check if the current allocator supports FMemory::GetAllocSize()
		void* TempMem = FMemory::Malloc(16);
		bool bSupportsGetAllocSize = FMemory::GetAllocSize(TempMem) >= 16;
		FMemory::Free(TempMem);

		// Use the workaround functions if GetAllocSize is not supported.
		sqlite3_config(SQLITE_CONFIG_MALLOC, bSupportsGetAllocSize ? &MallocFuncs : &WorkaroundMallocFuncs);
	}

private:
	/** Initialize the malloc system */
	static int Init(void*)
	{
		return SQLITE_OK;
	}

	/** Shutdown the malloc system */
	static void Shutdown(void*)
	{
	}

	/** Allocate memory */
	static void* Malloc(int InSizeBytes)
	{
		return FMemory::Malloc(InSizeBytes, DEFAULT_ALIGNMENT);
	}

	/** Free memory returned by Alloc or Realloc */
	static void Free(void* InPtr)
	{
		FMemory::Free(InPtr);
	}

	/** Reallocate memory returned by Alloc or Realloc */
	static void* Realloc(void* InPtr, int InSizeBytes)
	{
		return FMemory::Realloc(InPtr, InSizeBytes, DEFAULT_ALIGNMENT);
	}

	/** Get the actual size of an allocation returned by Alloc or Realloc */
	static int Size(void* InPtr)
	{
		return (int)FMemory::GetAllocSize(InPtr);
	}

	/** Roundup to the expected allocation size */
	static int Roundup(int InSizeBytes)
	{
		return (int)FMemory::QuantizeSize(InSizeBytes, DEFAULT_ALIGNMENT);
	}

	/**
	 * Workaround memory allocation functions which store the size in the 16 bytes preceding the allocation
	 * This assumes that the default alignment is no more than 16 bytes, so we can just subtract 16 from any non-null pointer to get to the size
	 */
	static constexpr int WorkaroundHeaderSize = 16;

	/** Allocate memory */
	static void* WorkaroundMalloc(int InSizeBytes)
	{
		void* Result = FMemory::Malloc(InSizeBytes + WorkaroundHeaderSize, DEFAULT_ALIGNMENT);
		if (Result)
		{
			*(int*)Result = InSizeBytes;
			Result = (char*)Result + WorkaroundHeaderSize;
		}
		return Result;
	}

	/** Free memory returned by Alloc or Realloc */
	static void WorkaroundFree(void* InPtr)
	{
		if (InPtr)
		{
			FMemory::Free((char*)InPtr - WorkaroundHeaderSize);
		}
	}

	/** Reallocate memory returned by Alloc or Realloc */
	static void* WorkaroundRealloc(void* InPtr, int InSizeBytes)
	{
		if (InPtr)
		{
			InPtr = (char*)InPtr - WorkaroundHeaderSize;
		}
		void* Result = FMemory::Realloc(InPtr, InSizeBytes + WorkaroundHeaderSize, DEFAULT_ALIGNMENT);
		if (Result)
		{
			*(int*)Result = InSizeBytes;
			Result = (char*)Result + WorkaroundHeaderSize;
		}
		return Result;
	}

	/** Get the actual size of an allocation returned by Alloc or Realloc */
	static int WorkaroundSize(void* InPtr)
	{
		return InPtr ? *(int*)((char*)InPtr - WorkaroundHeaderSize) : 0;
	}

	/** Roundup to the expected allocation size */
	static int WorkaroundRoundup(int InSizeBytes)
	{
		return (int)FMemory::QuantizeSize(InSizeBytes + WorkaroundHeaderSize, DEFAULT_ALIGNMENT) - WorkaroundHeaderSize;
	}
};

/** Unreal implementation of an SQLite mutex */
struct FSQLiteMutex
{
	FSQLiteMutex(int InSQLiteMutexId)
		: SQLiteMutexId(InSQLiteMutexId)
	{
	}

	FCriticalSection CriticalSection;
	int SQLiteMutexId;
#ifdef SQLITE_DEBUG
	TAtomic<uint32> OwnerThreadId{(uint32)INDEX_NONE};
#endif
};

/** Mutex functions used by SQLite (see sqlite3_mutex_methods) */
struct FSQLiteMutexFuncs
{
public:
	/** Register the mutex system */
	static void Register()
	{
		static const sqlite3_mutex_methods MutexFuncs = {
			&Init,
			&End,
			&Alloc,
			&Free,
			&Enter,
			&Try,
			&Leave,
			&Held,
			&Notheld
		};

		sqlite3_config(SQLITE_CONFIG_MUTEX, &MutexFuncs);
	}

private:
	/** Array of static mutexes used by SQLite */
	static const int32 SQLiteStaticMutexArrayCount = 12;
	static FSQLiteMutex* SQLiteStaticMutexArray[SQLiteStaticMutexArrayCount];

	/** Initialize the mutex system */
	static int Init()
	{
		for (int32 SQLiteStaticMutexIndex = 0; SQLiteStaticMutexIndex < SQLiteStaticMutexArrayCount; ++SQLiteStaticMutexIndex)
		{
			check(!SQLiteStaticMutexArray[SQLiteStaticMutexIndex]);
			SQLiteStaticMutexArray[SQLiteStaticMutexIndex] = new FSQLiteMutex(SQLiteStaticMutexIndex + 2);
		}

		return SQLITE_OK;
	}

	/** Shutdown the mutex system */
	static int End()
	{
		for (int32 SQLiteStaticMutexIndex = 0; SQLiteStaticMutexIndex < SQLiteStaticMutexArrayCount; ++SQLiteStaticMutexIndex)
		{
			check(SQLiteStaticMutexArray[SQLiteStaticMutexIndex]);
			delete SQLiteStaticMutexArray[SQLiteStaticMutexIndex];
			SQLiteStaticMutexArray[SQLiteStaticMutexIndex] = nullptr;
		}

		return SQLITE_OK;
	}

	/** Allocate a mutex */
	static sqlite3_mutex* Alloc(int InSQLiteMutexId)
	{
		if (InSQLiteMutexId == SQLITE_MUTEX_FAST || InSQLiteMutexId == SQLITE_MUTEX_RECURSIVE)
		{
			return (sqlite3_mutex*)new FSQLiteMutex(InSQLiteMutexId);
		}
		else
		{
			const int32 SQLiteStaticMutexIndex = InSQLiteMutexId - 2;
			check(SQLiteStaticMutexIndex >= 0 && SQLiteStaticMutexIndex < SQLiteStaticMutexArrayCount);

			FSQLiteMutex* SQLiteStaticMutex = SQLiteStaticMutexArray[SQLiteStaticMutexIndex];
			checkSlow(SQLiteStaticMutex->SQLiteMutexId == InSQLiteMutexId);

			return (sqlite3_mutex*)SQLiteStaticMutex;
		}
	}

	/** Free a mutex returned by Alloc */
	static void Free(sqlite3_mutex* InMutex)
	{
		FSQLiteMutex* Mutex = (FSQLiteMutex*)InMutex;
		check(Mutex);

		if (Mutex->SQLiteMutexId == SQLITE_MUTEX_FAST || Mutex->SQLiteMutexId == SQLITE_MUTEX_RECURSIVE)
		{
			delete Mutex;
		}
	}

	/** Lock a mutex returned by Alloc */
	static void Enter(sqlite3_mutex* InMutex)
	{
		FSQLiteMutex* Mutex = (FSQLiteMutex*)InMutex;
		check(Mutex);

		Mutex->CriticalSection.Lock();
#ifdef SQLITE_DEBUG
		Mutex->OwnerThreadId = FPlatformTLS::GetCurrentThreadId();
#endif
	}

	/** TryLock a mutex returned by Alloc */
	static int Try(sqlite3_mutex* InMutex)
	{
		FSQLiteMutex* Mutex = (FSQLiteMutex*)InMutex;
		check(Mutex);

		if (Mutex->CriticalSection.TryLock())
		{
#ifdef SQLITE_DEBUG
			Mutex->OwnerThreadId = FPlatformTLS::GetCurrentThreadId();
#endif
			return SQLITE_OK;
		}
		return SQLITE_BUSY;
	}

	/** Unlock a mutex returned by Alloc */
	static void Leave(sqlite3_mutex* InMutex)
	{
		FSQLiteMutex* Mutex = (FSQLiteMutex*)InMutex;
		check(Mutex);

		Mutex->CriticalSection.Unlock();
#ifdef SQLITE_DEBUG
		Mutex->OwnerThreadId = (uint32)INDEX_NONE;
#endif
	}

	/** Test whether a mutex returned by Alloc is held by the current thread (debug only) */
	static int Held(sqlite3_mutex* InMutex)
	{
		FSQLiteMutex* Mutex = (FSQLiteMutex*)InMutex;
		check(Mutex);

#ifdef SQLITE_DEBUG
		return Mutex->OwnerThreadId == FPlatformTLS::GetCurrentThreadId();
#else
		return true;
#endif
	}

	/** Test whether a mutex returned by Alloc is not held by the current thread (debug only) */
	static int Notheld(sqlite3_mutex* InMutex)
	{
		FSQLiteMutex* Mutex = (FSQLiteMutex*)InMutex;
		check(Mutex);

#ifdef SQLITE_DEBUG
		return Mutex->OwnerThreadId != FPlatformTLS::GetCurrentThreadId();
#else
		return true;
#endif
	}
};

FSQLiteMutex* FSQLiteMutexFuncs::SQLiteStaticMutexArray[FSQLiteMutexFuncs::SQLiteStaticMutexArrayCount] = { 0 };

/** Unreal implementation of an SQLite file (zeroed on init) */
struct FSQLiteFile
{
	const sqlite3_io_methods* IOMethods;
	IFileHandle* FileHandle;
	FString Filename;
	int LockMode;
	bool bDeleteOnClose;
	bool bIsReadOnly;
	
	static FCriticalSection CurrentlyOpenAsReadOnlySection;
	static TSet<FString> CurrentlyOpenAsReadOnly;
	static bool AllowOpenAsReadOnly(const TCHAR* InFilename);
	static void CloseAsReadOnly(const TCHAR* InFilename);
};

FCriticalSection FSQLiteFile::CurrentlyOpenAsReadOnlySection;
TSet<FString> FSQLiteFile::CurrentlyOpenAsReadOnly;

bool FSQLiteFile::AllowOpenAsReadOnly(const TCHAR* InFilename)
{
	IPlatformFile& PlatformFile = FPlatformFileManager::Get().GetPlatformFile();
	FString CanonFilename = PlatformFile.GetFilenameOnDisk(InFilename);

	// for consistency's sake we try our best to enforce only opening each file _once_
	//  if we open for write we obtain a lock to the file, but not for read-only
	//  instead implement a tracking list of files that have been opened
	bool bIsAlreadyInSet = false;
	FScopeLock Lock(&CurrentlyOpenAsReadOnlySection);
	bool bAdded = CurrentlyOpenAsReadOnly.Add(CanonFilename, &bIsAlreadyInSet).IsValidId();
	check(bAdded);

	// only succeed if we were the call to add it to the set
	return !bIsAlreadyInSet;
}

void FSQLiteFile::CloseAsReadOnly(const TCHAR* InFilename)
{
	IPlatformFile& PlatformFile = FPlatformFileManager::Get().GetPlatformFile();
	FString CanonFilename = PlatformFile.GetFilenameOnDisk(InFilename);
	FScopeLock Lock(&CurrentlyOpenAsReadOnlySection);
	int32 NumRemoved = CurrentlyOpenAsReadOnly.Remove(CanonFilename);
	check(NumRemoved > 0);
}

/**
 * File functions used by SQLite (see sqlite3_io_methods and sqlite3_vfs)
 * @note We have to make some concessions for things not exposed in the Unreal HAL that will affect multi-process concurrency (single-process access is not affected):
 *   - We do not provide an implementation for shared memory (mmap) as not all platforms implement it (see MapNamedSharedMemoryRegion and UnmapNamedSharedMemoryRegion)
 *   - We do not provide an implementation for granular file locks as our HAL doesn't expose the concept; instead we always take a writable handle for all file opens to prevent concurrent writes
 */
struct FSQLiteFileFuncs
{
public:
	/** Register the file system */
	static void Register()
	{
		static sqlite3_vfs VFSFuncs = {
			3,									/* VFS version number */
			sizeof(FSQLiteFile),				/* Size of sqlite3_file implementation for this VFS */
			FPlatformMisc::GetMaxPathLength(),	/* Maximum file pathname length */
			nullptr,							/* Next registered VFS (set internally) */
			"unreal-fs",						/* Name of this VFS */
			nullptr,							/* Pointer to application-specific data */
			&Open,
			&Delete,
			&Access,
			&FullPathname,
			nullptr,							/** Extension support disabled */
			nullptr,							/** Extension support disabled */
			nullptr,							/** Extension support disabled */
			nullptr,							/** Extension support disabled */
			&Randomness,
			&Sleep,
			&CurrentTime,
			&GetLastError,
			&CurrentTimeInt64,
			nullptr,							/** Unused debug function */
			nullptr,							/** Unused debug function */
			nullptr,							/** Unused debug function */
		};

		sqlite3_vfs_register(&VFSFuncs, 1);
	}

private:
	/** Attempt to open a file */
	static int Open(sqlite3_vfs* InVFS, const char* InFilename, sqlite3_file* InFile, int InFlags, int* OutFlagsPtr)
	{
		static const sqlite3_io_methods FileFuncs = {
			1,	/** Version 1, no shared memory support */
			&Close,
			&Read,
			&Write,
			&Truncate,
			&Sync,
			&FileSize,
			&Lock,
			&Unlock,
			&CheckReservedLock,
			&FileControl,
			&SectorSize,
			&DeviceCharacteristics,
		};

		FSQLiteFile* File = (FSQLiteFile*)InFile;
		check(File);

		// Zero the file descriptor so it has valid data for the early return cases
		FMemory::Memzero(*File);

		IPlatformFile& PlatformFile = FPlatformFileManager::Get().GetPlatformFile();

		// We have to make a temporary filename if given a null filename
		if (InFilename)
		{
			new (&File->Filename) FString(UTF8_TO_TCHAR(InFilename));
		}
		else
		{
			static const FString TmpPath = FPaths::ProjectIntermediateDir() / TEXT("SQLite");
			PlatformFile.CreateDirectory(*TmpPath);
			new (&File->Filename) FString(FPaths::CreateTempFilename(*TmpPath));
		}

		auto ReturnIOError = [&File]()
		{
			File->Filename.~FString();

			return SQLITE_IOERR;
		};

		// Does the file already exist? If so "exclusive" should fail. If not, anything that doesn't specify "create" should fail
		const bool bFileExists = PlatformFile.FileExists(*File->Filename);
		if (bFileExists)
		{
			if (InFlags & SQLITE_OPEN_EXCLUSIVE)
			{
				return ReturnIOError();
			}
		}
		else if (!(InFlags & SQLITE_OPEN_CREATE))
		{

			return ReturnIOError();
		}

		// Stat the file to fetch its write-ability.
		File->bIsReadOnly = PlatformFile.IsReadOnly(*File->Filename);
		if (!File->bIsReadOnly)
		{
			// The Unreal HAL doesn't support granular file locking so we always obtain a write handle to any file regardless of what SQLite asked for
			// This prevents concurrent access to the files and makes the locking operations a no-op (though we still track the requested lock mode)
			File->FileHandle = PlatformFile.OpenWrite(*File->Filename, /*bAppend*/true, /*bAllowRead*/true);
		}
		else if (InFlags & SQLITE_OPEN_READONLY)
		{
			if (FSQLiteFile::AllowOpenAsReadOnly(*File->Filename))
			{
				// If we are unable to open for write, the file could be stored in a read-only way (Pak)
				// If we only require read access, attempt to open the in read-only mode
				// This assumes there won't be any contention (it's impossible to open for write)
				File->FileHandle = PlatformFile.OpenRead(*File->Filename);

				// Our open operation failed for a reason, make sure to keep bookkeeping up to date
				if (!File->FileHandle)
				{
					FSQLiteFile::CloseAsReadOnly(*File->Filename);
				}
			}
		}

		if (File->FileHandle)
		{
			File->FileHandle->Seek(0);
		}
		else
		{
			return ReturnIOError();
		}

		// Opened the file - fill in the rest of the data
		File->IOMethods = &FileFuncs;
		File->bDeleteOnClose = !!(InFlags & SQLITE_OPEN_DELETEONCLOSE);

		// Set-up the output flags
		if (OutFlagsPtr)
		{
			if (File->bIsReadOnly)
			{
				*OutFlagsPtr = SQLITE_OPEN_READONLY;
			}
		}

		return SQLITE_OK;
	}

	/** Close a file previously opened by Open */
	static int Close(sqlite3_file* InFile)
	{
		FSQLiteFile* File = (FSQLiteFile*)InFile;
		check(File && File->FileHandle);

		// Make sure to bookeep our special read-only file list
		if (File->bIsReadOnly)
		{
			FSQLiteFile::CloseAsReadOnly(*File->Filename);
		}

		// Deleting the handle instance closes the file
		delete File->FileHandle;

		// Should we also delete it?
		if (File->bDeleteOnClose)
		{
			IPlatformFile& PlatformFile = FPlatformFileManager::Get().GetPlatformFile();
			PlatformFile.DeleteFile(*File->Filename);
		}

		File->Filename.~FString();

		return SQLITE_OK;
	}

	/** Read from a file previously opened by Open */
	static int Read(sqlite3_file* InFile, void* OutBuffer, int InReadAmountBytes, sqlite3_int64 InReadOffsetBytes)
	{
		FSQLiteFile* File = (FSQLiteFile*)InFile;
		check(File && File->FileHandle);

		// Zero the buffer first in-case of a short read
		FMemory::Memzero(OutBuffer, InReadAmountBytes);

		//Seek() returns differently on different platform when offset is out of bound, so check it ahead
		int64 iFileSize = File->FileHandle->Size();
		if (InReadOffsetBytes >= iFileSize) {
			return SQLITE_IOERR_SHORT_READ;
		}

		if (!File->FileHandle->Seek(InReadOffsetBytes))
		{
			return SQLITE_IOERR_SEEK;
		}

		if (!File->FileHandle->Read((uint8*)OutBuffer, InReadAmountBytes))
		{
			// Did this read fail because it ran out of data?
			const int64 CurrentReadPosition = File->FileHandle->Tell();
			const int64 CurrentFileSize = File->FileHandle->Size();
			return CurrentReadPosition >= CurrentFileSize
				? SQLITE_IOERR_SHORT_READ
				: SQLITE_IOERR_READ;
		}

		return SQLITE_OK;
	}

	/** Write to a file previously opened by Open */
	static int Write(sqlite3_file* InFile, const void* InBuffer, int InWriteAmountBytes, sqlite3_int64 InWriteOffsetBytes)
	{
		FSQLiteFile* File = (FSQLiteFile*)InFile;
		check(File && File->FileHandle);

		if (!File->FileHandle->Seek(InWriteOffsetBytes))
		{
			return SQLITE_IOERR_SEEK;
		}

		if (!File->FileHandle->Write((const uint8*)InBuffer, InWriteAmountBytes))
		{
			return SQLITE_IOERR_WRITE;
		}

		return SQLITE_OK;
	}

	/** Truncate a file previously opened by Open */
	static int Truncate(sqlite3_file* InFile, sqlite3_int64 InSizeBytes)
	{
		FSQLiteFile* File = (FSQLiteFile*)InFile;
		check(File && File->FileHandle);

		if (!File->FileHandle->Truncate(InSizeBytes))
		{
			return SQLITE_IOERR_TRUNCATE;
		}

		return SQLITE_OK;
	}

	/** Synchronize a file previously opened by Open */
	static int Sync(sqlite3_file* InFile, int InFlags)
	{
		FSQLiteFile* File = (FSQLiteFile*)InFile;
		check(File && File->FileHandle);

		const bool bFullFlush = (InFlags & 0x0F) == SQLITE_SYNC_FULL;
		if (!File->FileHandle->Flush(bFullFlush))
		{
			return SQLITE_IOERR_FSYNC;
		}

		return SQLITE_OK;
	}

	/** Get the size of a file previously opened by Open */
	static int FileSize(sqlite3_file* InFile, sqlite3_int64* OutSizePtr)
	{
		FSQLiteFile* File = (FSQLiteFile*)InFile;
		check(File && File->FileHandle);

		check(OutSizePtr);
		*OutSizePtr = File->FileHandle->Size();

		return SQLITE_OK;
	}

	/** Lock a file previously opened by Open */
	static int Lock(sqlite3_file* InFile, int InLockMode)
	{
		FSQLiteFile* File = (FSQLiteFile*)InFile;
		check(File && File->FileHandle);

		// See Open for the notes on locking
		if (InLockMode < File->LockMode)
		{
			return SQLITE_IOERR_LOCK;
		}
		File->LockMode = InLockMode;

		return SQLITE_OK;
	}

	/** Unlock a file previously opened by Open */
	static int Unlock(sqlite3_file* InFile, int InLockMode)
	{
		FSQLiteFile* File = (FSQLiteFile*)InFile;
		check(File && File->FileHandle);

		// See Open for the notes on locking
		if (InLockMode > File->LockMode)
		{
			return SQLITE_IOERR_UNLOCK;
		}
		File->LockMode = InLockMode;

		return SQLITE_OK;
	}

	/** Check for a lock on a file previously opened by Open */
	static int CheckReservedLock(sqlite3_file* InFile, int* OutIsLocked)
	{
		FSQLiteFile* File = (FSQLiteFile*)InFile;
		check(File && File->FileHandle);

		// See Open for the notes on locking
		check(OutIsLocked);
		*OutIsLocked = File->LockMode > 0;

		return SQLITE_OK;
	}

	/** Perform additional control operations on a file previously opened by Open */
	static int FileControl(sqlite3_file* InFile, int InOp, void* InOutOpData)
	{
		FSQLiteFile* File = (FSQLiteFile*)InFile;
		check(File && File->FileHandle);

		switch (InOp)
		{
		case SQLITE_FCNTL_LOCKSTATE:
			*(int*)InOutOpData = File->LockMode;
			return SQLITE_OK;

		default:
			break;
		}

		return SQLITE_NOTFOUND;
	}

	/** Get the underlying sector size of a file previously opened by Open */
	static int SectorSize(sqlite3_file* InFile)
	{
		FSQLiteFile* File = (FSQLiteFile*)InFile;
		check(File && File->FileHandle);

		return 4096;
	}

	/** Get the device characteristics of a file previously opened by Open */
	static int DeviceCharacteristics(sqlite3_file* InFile)
	{
		FSQLiteFile* File = (FSQLiteFile*)InFile;
		check(File && File->FileHandle);

		return SQLITE_IOCAP_UNDELETABLE_WHEN_OPEN;
	}

	/** Attempt to delete the named file */
	static int Delete(sqlite3_vfs* InVFS, const char* InFilename, int InSyncDir)
	{
		check(InFilename);
		const FString Filename = UTF8_TO_TCHAR(InFilename);

		IPlatformFile& PlatformFile = FPlatformFileManager::Get().GetPlatformFile();

		if (PlatformFile.FileExists(*Filename))
		{
			if (!PlatformFile.DeleteFile(*Filename))
			{
				return SQLITE_IOERR_DELETE;
			}
		}
		else if (PlatformFile.DirectoryExists(*Filename))
		{
			if (!PlatformFile.DeleteDirectory(*Filename))
			{
				return SQLITE_IOERR_DELETE;
			}
		}

		if (InSyncDir)
		{
			// TODO: Implement directory syncing support?
		}

		return SQLITE_OK;
	}

	/** Check whether the given file or directory exists on disk */
	static int Access(sqlite3_vfs* InVFS, const char* InFilename, int InAccessMode, int* OutResultPtr)
	{
		check(InFilename);
		const FString Filename = UTF8_TO_TCHAR(InFilename);

		IPlatformFile& PlatformFile = FPlatformFileManager::Get().GetPlatformFile();

		const FFileStatData StatData = PlatformFile.GetStatData(*Filename);
		if (InAccessMode == SQLITE_ACCESS_READWRITE)
		{
			check(OutResultPtr);
			*OutResultPtr = StatData.bIsValid && !StatData.bIsReadOnly;
		}
		else
		{
			check(OutResultPtr);
			*OutResultPtr = StatData.bIsValid;
		}

		return SQLITE_OK;
	}

	/** Convert the given filename to be absolute */
	static int FullPathname(sqlite3_vfs* InVFS, const char* InFilename, int InOutputBufferSizeBytes, char* InOutputBuffer)
	{
		check(InFilename && InOutputBuffer);

		const FString AbsoluteFilename = FPaths::ConvertRelativePathToFull(UTF8_TO_TCHAR(InFilename));
		FCStringAnsi::Strncpy(InOutputBuffer, TCHAR_TO_UTF8(*AbsoluteFilename), InOutputBufferSizeBytes);
		InOutputBuffer[InOutputBufferSizeBytes - 1] = 0;

		return SQLITE_OK;
	}

	/** Attempt to get the requested number of random bytes into the output buffer, returning the number of random bytes actually filled */
	static int Randomness(sqlite3_vfs* InVFS, int InNumRandomBytesRequested, char* InOutputBuffer)
	{
		if (InNumRandomBytesRequested > 0)
		{
			check(InOutputBuffer);

			FRandomStream RandomStream;
			RandomStream.GenerateNewSeed();

			// Generate 32-bit random integers and copy them into the output buffer until we've generated the correct amount of randomness data
			uint8* OutputBufferPtr = (uint8*)InOutputBuffer;
			int32 RandomBytesRemaining = InNumRandomBytesRequested;
			do
			{
				const uint32 RandomNumber = RandomStream.GetUnsignedInt();
				const int32 RandomBytesToCopy = FMath::Min<int32>(RandomBytesRemaining, sizeof(RandomNumber));
				FMemory::Memcpy(OutputBufferPtr, &RandomNumber, RandomBytesToCopy);
				OutputBufferPtr += RandomBytesToCopy;
				RandomBytesRemaining -= RandomBytesToCopy;
			}
			while (RandomBytesRemaining > 0);
		}

		return InNumRandomBytesRequested;
	}

	/** Attempt to sleep the calling thread for at least the requested number of microseconds, returning the number of microseconds actually slept */
	static int Sleep(sqlite3_vfs* InVFS, int InNumMicrosecondsRequested)
	{
		const float SleepSeconds = float(InNumMicrosecondsRequested) / 1000000.0f;
		FPlatformProcess::Sleep(SleepSeconds);
		return int(SleepSeconds * 1000000.0f);
	}

	/** Get the current time as a Julian Day value */
	static int CurrentTime(sqlite3_vfs* InVFS, double* OutTimePtr)
	{
		check(OutTimePtr);

		const FDateTime TimeNow = FDateTime::Now();
		*OutTimePtr = TimeNow.GetJulianDay();

		return SQLITE_OK;
	}

	/** Get the current time as a Julian Day value, multiplied the number of milliseconds in a 24-hour day (86400000) */
	static int CurrentTimeInt64(sqlite3_vfs* InVFS, sqlite3_int64* OutTimePtr)
	{
		check(OutTimePtr);

		const FDateTime TimeNow = FDateTime::Now();
		*OutTimePtr = sqlite3_int64(TimeNow.GetJulianDay() * 86400000.0);

		return SQLITE_OK;
	}

	/** Get the last OS error */
	static int GetLastError(sqlite3_vfs* InVFS, int InOutputBufferSizeBytes, char* InOutputBuffer)
	{
		if (InOutputBufferSizeBytes > 0)
		{
			check(InOutputBuffer);

			TCHAR ErrorBuffer[1024];
			FPlatformMisc::GetSystemErrorMessage(ErrorBuffer, UE_ARRAY_COUNT(ErrorBuffer), 0);
			FCStringAnsi::Strncpy(InOutputBuffer, TCHAR_TO_UTF8(ErrorBuffer), InOutputBufferSizeBytes);
			InOutputBuffer[InOutputBufferSizeBytes - 1] = 0;
		}

		return SQLITE_OK;
	}
};

extern "C"
{

/** Perform additional initialization during sqlite3_initialize */
SQLITE_API int sqlite3_os_init()
{
	FSQLiteFileFuncs::Register();
	return SQLITE_OK;
}

/** Perform additional shutdown during sqlite3_shutdown */
SQLITE_API int sqlite3_os_end()
{
	return SQLITE_OK;
}

}	// extern "C"

/** Perform additional configuration before calling sqlite3_initialize - called from FSQLiteCore::StartupModule (not a real SQLite API function) */
int sqlite3_ue_config()
{
	FSQLiteMallocFuncs::Register();
	FSQLiteMutexFuncs::Register();
	return SQLITE_OK;
}

#endif
