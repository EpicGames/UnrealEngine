// Copyright Epic Games, Inc. All Rights Reserved.

#include "SQLitePreparedStatement.h"
#include "SQLiteDatabase.h"
#include "IncludeSQLite.h"

#include "Misc/AssertionMacros.h"
#include "Containers/StringConv.h"
#include "Serialization/MemoryReader.h"
#include "Serialization/MemoryWriter.h"

DEFINE_LOG_CATEGORY_STATIC(LogSQLitePreparedStatement, Log, All);

FSQLitePreparedStatement::FSQLitePreparedStatement()
	: Statement(nullptr)
{
}

FSQLitePreparedStatement::FSQLitePreparedStatement(FSQLiteDatabase& InDatabase, const TCHAR* InStatement, const ESQLitePreparedStatementFlags InFlags)
	: Statement(nullptr)
{
	Create(InDatabase, InStatement, InFlags);
}

FSQLitePreparedStatement::~FSQLitePreparedStatement()
{
	Destroy();
}

FSQLitePreparedStatement::FSQLitePreparedStatement(FSQLitePreparedStatement&& Other)
	: Statement(Other.Statement)
	, CachedColumnNames(MoveTemp(Other.CachedColumnNames))
{
	Other.Statement = nullptr;
	Other.CachedColumnNames.Reset();
}

FSQLitePreparedStatement& FSQLitePreparedStatement::operator=(FSQLitePreparedStatement&& Other)
{
	if (this != &Other)
	{
		Destroy();

		Statement = Other.Statement;
		Other.Statement = nullptr;

		CachedColumnNames = MoveTemp(Other.CachedColumnNames);
		Other.CachedColumnNames.Reset();
	}
	return *this;
}

bool FSQLitePreparedStatement::IsValid() const
{
	return Statement != nullptr;
}

bool FSQLitePreparedStatement::IsActive() const
{
	return Statement && sqlite3_stmt_busy(Statement) != 0;
}

bool FSQLitePreparedStatement::IsReadOnly() const
{
	return Statement && sqlite3_stmt_readonly(Statement) != 0;
}

bool FSQLitePreparedStatement::Create(FSQLiteDatabase& InDatabase, const TCHAR* InStatement, const ESQLitePreparedStatementFlags InFlags)
{
	checkf(InDatabase.IsValid(), TEXT("Tried to create a prepared statement from an invalid database!"));

	if (Statement)
	{
		return false;
	}

	int32 PrepFlags = 0;
	if (EnumHasAnyFlags(InFlags, ESQLitePreparedStatementFlags::Persistent))
	{
		PrepFlags |= SQLITE_PREPARE_PERSISTENT;
	}

	if (sqlite3_prepare_v3(InDatabase.Database, TCHAR_TO_UTF8(InStatement), -1, PrepFlags, &Statement, nullptr) != SQLITE_OK)
	{
		UE_LOGF(LogSQLitePreparedStatement, Warning, "Failed to create prepared statement from '%ls': %ls", InStatement, *InDatabase.GetLastError());
		return false;
	}

	return true;
}

bool FSQLitePreparedStatement::Destroy()
{
	if (!Statement)
	{
		return false;
	}

	sqlite3_finalize(Statement);
	Statement = nullptr;

	CachedColumnNames.Reset();

	return true;
}

void FSQLitePreparedStatement::Reset()
{
	if (!Statement)
	{
		return;
	}

	sqlite3_reset(Statement);
}

void FSQLitePreparedStatement::ClearBindings()
{
	if (!Statement)
	{
		return;
	}

	sqlite3_clear_bindings(Statement);
}

int32 FSQLitePreparedStatement::GetBindingIndexByName(const TCHAR* InBindingName) const
{
	if (!Statement)
	{
		return 0;
	}

	return sqlite3_bind_parameter_index(Statement, TCHAR_TO_UTF8(InBindingName));
}

template <typename T>
bool FSQLitePreparedStatement::SetBindingValueByName_Integer(const TCHAR* InBindingName, const T InValue)
{
	return SetBindingValueByIndex_Integer(GetBindingIndexByName(InBindingName), InValue);
}

template <typename T>
bool FSQLitePreparedStatement::SetBindingValueByIndex_Integer(const int32 InBindingIndex, const T InValue)
{
	if (!Statement || InBindingIndex < 1)
	{
		return false;
	}

	return sqlite3_bind_int64(Statement, InBindingIndex, (sqlite3_int64)InValue) == SQLITE_OK;
}

bool FSQLitePreparedStatement::SetBindingValueByName(const TCHAR* InBindingName, const int8 InValue)
{
	return SetBindingValueByName_Integer(InBindingName, InValue);
}

bool FSQLitePreparedStatement::SetBindingValueByIndex(const int32 InBindingIndex, const int8 InValue)
{
	return SetBindingValueByIndex_Integer(InBindingIndex, InValue);
}

bool FSQLitePreparedStatement::SetBindingValueByName(const TCHAR* InBindingName, const uint8 InValue)
{
	return SetBindingValueByName_Integer(InBindingName, InValue);
}

bool FSQLitePreparedStatement::SetBindingValueByIndex(const int32 InBindingIndex, const uint8 InValue)
{
	return SetBindingValueByIndex_Integer(InBindingIndex, InValue);
}

bool FSQLitePreparedStatement::SetBindingValueByName(const TCHAR* InBindingName, const int16 InValue)
{
	return SetBindingValueByName_Integer(InBindingName, InValue);
}

bool FSQLitePreparedStatement::SetBindingValueByIndex(const int32 InBindingIndex, const int16 InValue)
{
	return SetBindingValueByIndex_Integer(InBindingIndex, InValue);
}

bool FSQLitePreparedStatement::SetBindingValueByName(const TCHAR* InBindingName, const uint16 InValue)
{
	return SetBindingValueByName_Integer(InBindingName, InValue);
}

bool FSQLitePreparedStatement::SetBindingValueByIndex(const int32 InBindingIndex, const uint16 InValue)
{
	return SetBindingValueByIndex_Integer(InBindingIndex, InValue);
}

bool FSQLitePreparedStatement::SetBindingValueByName(const TCHAR* InBindingName, const int32 InValue)
{
	return SetBindingValueByName_Integer(InBindingName, InValue);
}

bool FSQLitePreparedStatement::SetBindingValueByIndex(const int32 InBindingIndex, const int32 InValue)
{
	return SetBindingValueByIndex_Integer(InBindingIndex, InValue);
}

bool FSQLitePreparedStatement::SetBindingValueByName(const TCHAR* InBindingName, const uint32 InValue)
{
	return SetBindingValueByName_Integer(InBindingName, InValue);
}

bool FSQLitePreparedStatement::SetBindingValueByIndex(const int32 InBindingIndex, const uint32 InValue)
{
	return SetBindingValueByIndex_Integer(InBindingIndex, InValue);
}

bool FSQLitePreparedStatement::SetBindingValueByName(const TCHAR* InBindingName, const int64 InValue)
{
	return SetBindingValueByName_Integer(InBindingName, InValue);
}

bool FSQLitePreparedStatement::SetBindingValueByIndex(const int32 InBindingIndex, const int64 InValue)
{
	return SetBindingValueByIndex_Integer(InBindingIndex, InValue);
}

bool FSQLitePreparedStatement::SetBindingValueByName(const TCHAR* InBindingName, const uint64 InValue)
{
	return SetBindingValueByName_Integer(InBindingName, InValue);
}

bool FSQLitePreparedStatement::SetBindingValueByIndex(const int32 InBindingIndex, const uint64 InValue)
{
	return SetBindingValueByIndex_Integer(InBindingIndex, InValue);
}

bool FSQLitePreparedStatement::SetBindingValueByName(const TCHAR* InBindingName, const FDateTime InValue)
{
	return SetBindingValueByIndex(GetBindingIndexByName(InBindingName), InValue);
}

bool FSQLitePreparedStatement::SetBindingValueByIndex(const int32 InBindingIndex, const FDateTime InValue)
{
	return SetBindingValueByIndex(InBindingIndex, InValue.ToUnixTimestamp());
}

template <typename T>
bool FSQLitePreparedStatement::SetBindingValueByName_Real(const TCHAR* InBindingName, const T InValue)
{
	return SetBindingValueByIndex_Real(GetBindingIndexByName(InBindingName), InValue);
}

template <typename T>
bool FSQLitePreparedStatement::SetBindingValueByIndex_Real(const int32 InBindingIndex, const T InValue)
{
	if (!Statement || InBindingIndex < 1)
	{
		return false;
	}

	return sqlite3_bind_double(Statement, InBindingIndex, (double)InValue) == SQLITE_OK;
}

bool FSQLitePreparedStatement::SetBindingValueByName(const TCHAR* InBindingName, const float InValue)
{
	return SetBindingValueByName_Real(InBindingName, InValue);
}

bool FSQLitePreparedStatement::SetBindingValueByIndex(const int32 InBindingIndex, const float InValue)
{
	return SetBindingValueByIndex_Real(InBindingIndex, InValue);
}

bool FSQLitePreparedStatement::SetBindingValueByName(const TCHAR* InBindingName, const double InValue)
{
	return SetBindingValueByName_Real(InBindingName, InValue);
}

bool FSQLitePreparedStatement::SetBindingValueByIndex(const int32 InBindingIndex, const double InValue)
{
	return SetBindingValueByIndex_Real(InBindingIndex, InValue);
}

bool FSQLitePreparedStatement::SetBindingValueByName(const TCHAR* InBindingName, const TCHAR* InValue)
{
	return SetBindingValueByIndex(GetBindingIndexByName(InBindingName), InValue);
}

bool FSQLitePreparedStatement::SetBindingValueByIndex(const int32 InBindingIndex, const TCHAR* InValue)
{
	if (!Statement || InBindingIndex < 1)
	{
		return false;
	}

	return sqlite3_bind_text(Statement, InBindingIndex, TCHAR_TO_UTF8(InValue), -1, SQLITE_TRANSIENT) == SQLITE_OK;
}

bool FSQLitePreparedStatement::SetBindingValueByName(const TCHAR* InBindingName, const FString& InValue)
{
	return SetBindingValueByName(InBindingName, *InValue);
}

bool FSQLitePreparedStatement::SetBindingValueByIndex(const int32 InBindingIndex, const FString& InValue)
{
	return SetBindingValueByIndex(InBindingIndex, *InValue);
}

bool FSQLitePreparedStatement::SetBindingValueByName(const TCHAR* InBindingName, const FName InValue)
{
	return SetBindingValueByName(InBindingName, *InValue.ToString());
}

bool FSQLitePreparedStatement::SetBindingValueByIndex(const int32 InBindingIndex, const FName InValue)
{
	return SetBindingValueByIndex(InBindingIndex, *InValue.ToString());
}

bool FSQLitePreparedStatement::SetBindingValueByName(const TCHAR* InBindingName, const FText& InValue)
{
	FString TextStr;
	FTextStringHelper::WriteToBuffer(TextStr, InValue);
	return SetBindingValueByName(InBindingName, TextStr);
}

bool FSQLitePreparedStatement::SetBindingValueByIndex(const int32 InBindingIndex, const FText& InValue)
{
	FString TextStr;
	FTextStringHelper::WriteToBuffer(TextStr, InValue);
	return SetBindingValueByIndex(InBindingIndex, TextStr);
}

bool FSQLitePreparedStatement::SetBindingValueByName(const TCHAR* InBindingName, TArrayView<const uint8> InBlobData, const bool bCopy)
{
	return SetBindingValueByName(InBindingName, InBlobData.GetData(), InBlobData.Num(), bCopy);
}

bool FSQLitePreparedStatement::SetBindingValueByIndex(const int32 InBindingIndex, TArrayView<const uint8> InBlobData, const bool bCopy)
{
	return SetBindingValueByIndex(InBindingIndex, InBlobData.GetData(), InBlobData.Num(), bCopy);
}

bool FSQLitePreparedStatement::SetBindingValueByName(const TCHAR* InBindingName, const void* InBlobData, const int32 InBlobDataSizeBytes, const bool bCopy)
{
	return SetBindingValueByIndex(GetBindingIndexByName(InBindingName), InBlobData, InBlobDataSizeBytes, bCopy);
}

bool FSQLitePreparedStatement::SetBindingValueByIndex(const int32 InBindingIndex, const void* InBlobData, const int32 InBlobDataSizeBytes, const bool bCopy)
{
	if (!Statement || InBindingIndex < 1)
	{
		return false;
	}

	return sqlite3_bind_blob(Statement, InBindingIndex, InBlobData, InBlobDataSizeBytes, bCopy ? SQLITE_TRANSIENT : SQLITE_STATIC) == SQLITE_OK;
}

bool FSQLitePreparedStatement::SetBindingValueByName(const TCHAR* InBindingName, const FGuid& InValue)
{
	return SetBindingValueByIndex(GetBindingIndexByName(InBindingName), InValue);
}

bool FSQLitePreparedStatement::SetBindingValueByIndex(const int32 InBindingIndex, const FGuid& InValue)
{
	TArray<uint8> GuidBytes;
	{
		FMemoryWriter GuidWriter(GuidBytes);
		GuidWriter << const_cast<FGuid&>(InValue);
	}
	return SetBindingValueByIndex(InBindingIndex, GuidBytes, true);
}

bool FSQLitePreparedStatement::SetBindingValueByName(const TCHAR* InBindingName)
{
	return SetBindingValueByIndex(GetBindingIndexByName(InBindingName));
}

bool FSQLitePreparedStatement::SetBindingValueByIndex(const int32 InBindingIndex)
{
	if (!Statement || InBindingIndex < 1)
	{
		return false;
	}

	return sqlite3_bind_null(Statement, InBindingIndex) == SQLITE_OK;
}

bool FSQLitePreparedStatement::Execute()
{
	return Execute([](const FSQLitePreparedStatement&)
	{
		return ESQLitePreparedStatementExecuteRowResult::Continue; // Execute everything
	}) != INDEX_NONE;
}

int64 FSQLitePreparedStatement::Execute(TFunctionRef<ESQLitePreparedStatementExecuteRowResult(const FSQLitePreparedStatement&)> InCallback)
{
	TRACE_CPUPROFILER_EVENT_SCOPE_STR("FSQLitePreparedStatement::Execute");
	checkf(IsValid() && !IsActive(), TEXT("SQLite statement must be valid and not-active!"));

	// Step it to completion (or error)
	int64 RowCount = 0;
	ESQLitePreparedStatementStepResult StepResult = ESQLitePreparedStatementStepResult::Done;
	while ((StepResult = Step()) == ESQLitePreparedStatementStepResult::Row)
	{
		++RowCount;

		const ESQLitePreparedStatementExecuteRowResult RowResult = InCallback(*this);
		if (RowResult != ESQLitePreparedStatementExecuteRowResult::Continue)
		{
			if (RowResult == ESQLitePreparedStatementExecuteRowResult::Error)
			{
				StepResult = ESQLitePreparedStatementStepResult::Error;
			}
			break;
		}
	}

	Reset();

	return StepResult == ESQLitePreparedStatementStepResult::Error
		? INDEX_NONE
		: RowCount;
}

ESQLitePreparedStatementStepResult FSQLitePreparedStatement::Step(int32* OutErrorCode)
{
	if (OutErrorCode)
	{
		*OutErrorCode = SQLITE_OK;
	}

	if (!Statement)
	{
		if (OutErrorCode)
		{
			*OutErrorCode = SQLITE_ERROR;
		}
		return ESQLitePreparedStatementStepResult::Error;
	}

	const int32 Result = sqlite3_step(Statement);
	switch (Result & 0xff) // Mask the result to basic error codes in case the database is using extended error codes
	{
	case SQLITE_ROW:
		return ESQLitePreparedStatementStepResult::Row;
	case SQLITE_DONE:
		return ESQLitePreparedStatementStepResult::Done;
	case SQLITE_BUSY:
		return ESQLitePreparedStatementStepResult::Busy;
	default:
		if (OutErrorCode)
		{
			*OutErrorCode = Result;
		}
		return ESQLitePreparedStatementStepResult::Error;
	}
}

int32 FSQLitePreparedStatement::GetColumnIndexByName(const TCHAR* InColumnName) const
{
	CacheColumnNames();
	return CachedColumnNames.IndexOfByPredicate([&InColumnName](const FString& InPotentialMatch)
	{
		return FCString::Stricmp(*InPotentialMatch, InColumnName) == 0; // Column names are case-insensitive
	});
}

template <typename T>
bool FSQLitePreparedStatement::GetColumnValueByName_Integer(const TCHAR* InColumnName, T& OutValue) const
{
	return GetColumnValueByIndex_Integer(GetColumnIndexByName(InColumnName), OutValue);
}

template <typename T>
bool FSQLitePreparedStatement::GetColumnValueByIndex_Integer(const int32 InColumnIndex, T& OutValue) const
{
	if (!Statement || !IsValidColumnIndex(InColumnIndex))
	{
		return false;
	}

	OutValue = (T)sqlite3_column_int64(Statement, InColumnIndex);
	return true;
}

bool FSQLitePreparedStatement::GetColumnValueByName(const TCHAR* InColumnName, int8& OutValue) const
{
	return GetColumnValueByName_Integer(InColumnName, OutValue);
}

bool FSQLitePreparedStatement::GetColumnValueByIndex(const int32 InColumnIndex, int8& OutValue) const
{
	return GetColumnValueByIndex_Integer(InColumnIndex, OutValue);
}

bool FSQLitePreparedStatement::GetColumnValueByName(const TCHAR* InColumnName, uint8& OutValue) const
{
	return GetColumnValueByName_Integer(InColumnName, OutValue);
}

bool FSQLitePreparedStatement::GetColumnValueByIndex(const int32 InColumnIndex, uint8& OutValue) const
{
	return GetColumnValueByIndex_Integer(InColumnIndex, OutValue);
}

bool FSQLitePreparedStatement::GetColumnValueByName(const TCHAR* InColumnName, int16& OutValue) const
{
	return GetColumnValueByName_Integer(InColumnName, OutValue);
}

bool FSQLitePreparedStatement::GetColumnValueByIndex(const int32 InColumnIndex, int16& OutValue) const
{
	return GetColumnValueByIndex_Integer(InColumnIndex, OutValue);
}

bool FSQLitePreparedStatement::GetColumnValueByName(const TCHAR* InColumnName, uint16& OutValue) const
{
	return GetColumnValueByName_Integer(InColumnName, OutValue);
}

bool FSQLitePreparedStatement::GetColumnValueByIndex(const int32 InColumnIndex, uint16& OutValue) const
{
	return GetColumnValueByIndex_Integer(InColumnIndex, OutValue);
}

bool FSQLitePreparedStatement::GetColumnValueByName(const TCHAR* InColumnName, int32& OutValue) const
{
	return GetColumnValueByName_Integer(InColumnName, OutValue);
}

bool FSQLitePreparedStatement::GetColumnValueByIndex(const int32 InColumnIndex, int32& OutValue) const
{
	return GetColumnValueByIndex_Integer(InColumnIndex, OutValue);
}

bool FSQLitePreparedStatement::GetColumnValueByName(const TCHAR* InColumnName, uint32& OutValue) const
{
	return GetColumnValueByName_Integer(InColumnName, OutValue);
}

bool FSQLitePreparedStatement::GetColumnValueByIndex(const int32 InColumnIndex, uint32& OutValue) const
{
	return GetColumnValueByIndex_Integer(InColumnIndex, OutValue);
}

bool FSQLitePreparedStatement::GetColumnValueByName(const TCHAR* InColumnName, int64& OutValue) const
{
	return GetColumnValueByName_Integer(InColumnName, OutValue);
}

bool FSQLitePreparedStatement::GetColumnValueByIndex(const int32 InColumnIndex, int64& OutValue) const
{
	return GetColumnValueByIndex_Integer(InColumnIndex, OutValue);
}

bool FSQLitePreparedStatement::GetColumnValueByName(const TCHAR* InColumnName, uint64& OutValue) const
{
	return GetColumnValueByName_Integer(InColumnName, OutValue);
}

bool FSQLitePreparedStatement::GetColumnValueByIndex(const int32 InColumnIndex, uint64& OutValue) const
{
	return GetColumnValueByIndex_Integer(InColumnIndex, OutValue);
}

bool FSQLitePreparedStatement::GetColumnValueByName(const TCHAR* InColumnName, FDateTime& OutValue) const
{
	return GetColumnValueByIndex(GetColumnIndexByName(InColumnName), OutValue);
}

bool FSQLitePreparedStatement::GetColumnValueByIndex(const int32 InColumnIndex, FDateTime& OutValue) const
{
	int64 UnixTimestamp = 0;
	if (GetColumnValueByIndex(InColumnIndex, UnixTimestamp))
	{
		OutValue = FDateTime::FromUnixTimestamp(UnixTimestamp);
		return true;
	}
	return false;
}

template <typename T>
bool FSQLitePreparedStatement::GetColumnValueByName_Real(const TCHAR* InColumnName, T& OutValue) const
{
	return GetColumnValueByIndex_Real(GetColumnIndexByName(InColumnName), OutValue);
}

template <typename T>
bool FSQLitePreparedStatement::GetColumnValueByIndex_Real(const int32 InColumnIndex, T& OutValue) const
{
	if (!Statement || !IsValidColumnIndex(InColumnIndex))
	{
		return false;
	}

	OutValue = (T)sqlite3_column_double(Statement, InColumnIndex);
	return true;
}

bool FSQLitePreparedStatement::GetColumnValueByName(const TCHAR* InColumnName, float& OutValue) const
{
	return GetColumnValueByName_Real(InColumnName, OutValue);
}

bool FSQLitePreparedStatement::GetColumnValueByIndex(const int32 InColumnIndex, float& OutValue) const
{
	return GetColumnValueByIndex_Real(InColumnIndex, OutValue);
}

bool FSQLitePreparedStatement::GetColumnValueByName(const TCHAR* InColumnName, double& OutValue) const
{
	return GetColumnValueByName_Real(InColumnName, OutValue);
}

bool FSQLitePreparedStatement::GetColumnValueByIndex(const int32 InColumnIndex, double& OutValue) const
{
	return GetColumnValueByIndex_Real(InColumnIndex, OutValue);
}

bool FSQLitePreparedStatement::GetColumnValueByName(const TCHAR* InColumnName, FString& OutValue) const
{
	return GetColumnValueByIndex(GetColumnIndexByName(InColumnName), OutValue);
}

bool FSQLitePreparedStatement::GetColumnValueByIndex(const int32 InColumnIndex, FString& OutValue) const
{
	if (!Statement || !IsValidColumnIndex(InColumnIndex))
	{
		return false;
	}

	const char* ColumnValueUTF8 = (const char*)sqlite3_column_text(Statement, InColumnIndex);
	OutValue = ColumnValueUTF8 ? UTF8_TO_TCHAR(ColumnValueUTF8) : TEXT("");
	return true;
}

bool FSQLitePreparedStatement::GetColumnValueByName(const TCHAR* InColumnName, FName& OutValue) const
{
	return GetColumnValueByIndex(GetColumnIndexByName(InColumnName), OutValue);
}

bool FSQLitePreparedStatement::GetColumnValueByIndex(const int32 InColumnIndex, FName& OutValue) const
{
	if (!Statement || !IsValidColumnIndex(InColumnIndex))
	{
		return false;
	}

	const char* ColumnValueUTF8 = (const char*)sqlite3_column_text(Statement, InColumnIndex);
	OutValue = FName(ColumnValueUTF8 ? UTF8_TO_TCHAR(ColumnValueUTF8) : TEXT(""));
	return true;
}

bool FSQLitePreparedStatement::GetColumnValueByName(const TCHAR* InColumnName, FText& OutValue) const
{
	return GetColumnValueByIndex(GetColumnIndexByName(InColumnName), OutValue);
}

bool FSQLitePreparedStatement::GetColumnValueByIndex(const int32 InColumnIndex, FText& OutValue) const
{
	if (!Statement || !IsValidColumnIndex(InColumnIndex))
	{
		return false;
	}

	const char* ColumnValueUTF8 = (const char*)sqlite3_column_text(Statement, InColumnIndex);
	OutValue = FTextStringHelper::CreateFromBuffer(ColumnValueUTF8 ? UTF8_TO_TCHAR(ColumnValueUTF8) : TEXT(""));
	return true;
}

bool FSQLitePreparedStatement::GetColumnValueByName(const TCHAR* InColumnName, TArray<uint8>& OutValue) const
{
	return GetColumnValueByIndex(GetColumnIndexByName(InColumnName), OutValue);
}

bool FSQLitePreparedStatement::GetColumnValueByIndex(const int32 InColumnIndex, TArray<uint8>& OutValue) const
{
	if (!Statement || !IsValidColumnIndex(InColumnIndex))
	{
		return false;
	}

	const uint8* ColumnValueBlob = (const uint8*)sqlite3_column_blob(Statement, InColumnIndex);
	const int32 ColumnValueBlobSizeBytes = sqlite3_column_bytes(Statement, InColumnIndex);

	OutValue.Reset();
	OutValue.Append(ColumnValueBlob, ColumnValueBlobSizeBytes);

	return true;
}

bool FSQLitePreparedStatement::GetColumnValueByName(const TCHAR* InColumnName, FGuid& OutValue) const
{
	return GetColumnValueByIndex(GetColumnIndexByName(InColumnName), OutValue);
}

bool FSQLitePreparedStatement::GetColumnValueByIndex(const int32 InColumnIndex, FGuid& OutValue) const
{
	TArray<uint8> GuidBytes;
	if (GetColumnValueByIndex(InColumnIndex, GuidBytes))
	{
		FMemoryReader GuidReader(GuidBytes);
		GuidReader << OutValue;
		return !GuidReader.GetError();
	}
	return false;
}

bool FSQLitePreparedStatement::GetColumnTypeByName(const TCHAR* InColumnName, ESQLiteColumnType& OutColumnType) const
{
	return GetColumnTypeByIndex(GetColumnIndexByName(InColumnName), OutColumnType);
}

bool FSQLitePreparedStatement::GetColumnTypeByIndex(const int32 InColumnIndex, ESQLiteColumnType& OutColumnType) const
{
	if (!Statement || !IsValidColumnIndex(InColumnIndex))
	{
		return false;
	}

#define SQLITE_COLUMN_TYPE_TO_WRAPPED_TYPE(A, B) case A: OutColumnType = B; break
	const int ColumnTypeId = sqlite3_column_type(Statement, InColumnIndex);
	switch (ColumnTypeId)
	{
		SQLITE_COLUMN_TYPE_TO_WRAPPED_TYPE(SQLITE_INTEGER, ESQLiteColumnType::Integer);
		SQLITE_COLUMN_TYPE_TO_WRAPPED_TYPE(SQLITE_FLOAT, ESQLiteColumnType::Float);
		SQLITE_COLUMN_TYPE_TO_WRAPPED_TYPE(SQLITE_TEXT, ESQLiteColumnType::String);
		SQLITE_COLUMN_TYPE_TO_WRAPPED_TYPE(SQLITE_BLOB, ESQLiteColumnType::Blob);
		SQLITE_COLUMN_TYPE_TO_WRAPPED_TYPE(SQLITE_NULL, ESQLiteColumnType::Null);
	default:
		break;
	}
#undef SQLITE_COLUMN_TYPE_TO_WRAPPED_TYPE

	return true;
}

const TArray<FString>& FSQLitePreparedStatement::GetColumnNames() const
{
	CacheColumnNames();
	return CachedColumnNames;
}

void FSQLitePreparedStatement::CacheColumnNames() const
{
	if (!Statement || CachedColumnNames.Num() > 0)
	{
		return;
	}

	const int32 ColumnCount = sqlite3_column_count(Statement);
	CachedColumnNames.Reserve(ColumnCount);

	for (int32 ColumnIndex = 0; ColumnIndex < ColumnCount; ++ColumnIndex)
	{
		const char* ColumnNameUTF8 = sqlite3_column_name(Statement, ColumnIndex);
		CachedColumnNames.Emplace(ColumnNameUTF8 ? UTF8_TO_TCHAR(ColumnNameUTF8) : TEXT(""));
	}
}

bool FSQLitePreparedStatement::IsValidColumnIndex(const int32 InColumnIndex) const
{
	CacheColumnNames();
	return InColumnIndex >= 0 && InColumnIndex < CachedColumnNames.Num();
}
