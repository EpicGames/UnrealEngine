// Copyright Epic Games, Inc. All Rights Reserved.

#include "CoreMinimal.h"
#include "HAL/FileManager.h"
#include "Misc/AutomationTest.h"
#include "Misc/Paths.h"
#include "SQLiteDatabase.h"

#if WITH_DEV_AUTOMATION_TESTS

IMPLEMENT_SIMPLE_AUTOMATION_TEST(FSQLiteCoreTest, "System.Plugins.Database.SQLiteCore", EAutomationTestFlags::EditorContext | EAutomationTestFlags::EngineFilter)


/**
 * Performs very basic SQLite testing, not to test SQLite itself, but to ensure the UE5 abstraction layer works appropriately. If this test fails, this is
 * likely because something in \Engine\Plugins\Runtime\Database\SQLiteCore\Source\SQLiteCore\Private\SQLiteEmbeddedPlatform.cpp is broken. The usual suspects are:
 * 
 * static int Open(sqlite3_vfs* InVFS, const char* InFilename, sqlite3_file* InFile, int InFlags, int* OutFlagsPtr)
 * static int Read(sqlite3_file* InFile, void* OutBuffer, int InReadAmountBytes, sqlite3_int64 InReadOffsetBytes)
 * static int Write(sqlite3_file* InFile, const void* InBuffer, int InWriteAmountBytes, sqlite3_int64 InWriteOffsetBytes)
 */
bool FSQLiteCoreTest::RunTest(const FString& Parameters)
{
	FString Path = FPaths::ConvertRelativePathToFull(FPaths::AutomationTransientDir() / TEXT("SQLiteTests") / "SQLiteTest.db");
	IFileManager::Get().Delete(*Path);
	bool bSuccess = true;

	// Try to open a file that doesn't exist, ensure that close cannot be called or doesn't execute unexpectedly
	{
		FSQLiteDatabase TestDb;
		bSuccess &= !TestDb.Open(*Path, ESQLiteDatabaseOpenMode::ReadWrite);	// Expected to return false since the file does not exist on disk at this time
		bSuccess &= !TestDb.Close();	// Expected to return false if nothing was opened
	}

	// Utility function to read back the records and check if the one expected.
	auto ReadBackRecordsAndValidateValues = [](FSQLiteDatabase& Db)
	{
		bool bSucceeded = true;

		// Read back the Records.
		FString SelectStatement = TEXT("SELECT * FROM users ORDER BY id");
		Db.Execute(*SelectStatement, [&bSucceeded](const FSQLitePreparedStatement& Callback)
		{
			int64 Id;
			FString Name;
			FString Title;
			Callback.GetColumnValueByName(TEXT("id"), Id);
			Callback.GetColumnValueByName(TEXT("name"), Name);
			Callback.GetColumnValueByName(TEXT("title"), Title);

			bSucceeded &= (Id == 1);
			bSucceeded &= (Name == TEXT("John"));
			bSucceeded &= (Title == TEXT("Manager"));

			return ESQLitePreparedStatementExecuteRowResult::Stop;
		});

		return bSucceeded;
	};

	// Exercise very basic SQLite HAL operations (open, close, seek, lock, unlock, etc)
	{
		FSQLiteDatabase TestDb;
		bSuccess &= TestDb.Open(*Path, ESQLiteDatabaseOpenMode::ReadWriteCreate);

		// Create a table.
		FString Statement = TEXT("CREATE TABLE users (id INTEGER NOT NULL,name TEXT, title TEXT)");
		bSuccess &= TestDb.Execute(*Statement);

		// Insert few records.
		Statement = TEXT("INSERT INTO users (id, name, title) VALUES (1, 'John', 'Manager')");
		bSuccess &= TestDb.Execute(*Statement);
		Statement = TEXT("INSERT INTO users (id, name, title) VALUES (2, 'Mark', 'Engineer')");
		bSuccess &= TestDb.Execute(*Statement);

		// Read back the Records.
		bSuccess &= ReadBackRecordsAndValidateValues(TestDb);
		bSuccess &= TestDb.Close();
	}

	// Check if the database reloads correctly.
	{
		FSQLiteDatabase TestDb;
		bSuccess &= TestDb.Open(*Path, ESQLiteDatabaseOpenMode::ReadWriteCreate);
		bSuccess &= ReadBackRecordsAndValidateValues(TestDb);
		TestDb.Close();
	}

	IFileManager::Get().Delete(*Path);

	return bSuccess;
}

#endif // WITH_DEV_AUTOMATION_TESTS