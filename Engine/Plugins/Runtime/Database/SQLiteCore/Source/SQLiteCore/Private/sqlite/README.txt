This file contains the SQLite amalgamation that we embed into the SQLiteCore module.

To change the version of SQLite that is embedded, you need to:
    1) Go to https://www.sqlite.org/download.html and download a new amalgamation.
	2) Copy sqlite3.h into SQLiteCore\Public\sqlite and sqlite3.c into SQLiteCore\Private\sqlite.
	3) Rename sqlite3.c to sqlite3.inl.
