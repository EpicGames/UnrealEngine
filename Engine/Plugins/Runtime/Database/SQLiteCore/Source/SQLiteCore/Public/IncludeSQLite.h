// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreTypes.h"

THIRD_PARTY_INCLUDES_START
#include "sqlite/sqlite3.h"
THIRD_PARTY_INCLUDES_END
