// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreTypes.h"
#include "SQLitePreparedStatement.h"

#define UE_API SQLITECORE_API

/**
 * Modes used when opening a database.
 */
enum class ESQLiteDatabaseOpenMode : uint8
{
	/** Open the database in read-only mode. Fails if the database doesn't exist. */
	ReadOnly,

	/** Open the database in read-write mode if possible, or read-only mode if the database is write protected. Fails if the database doesn't exist. */
	ReadWrite,

	/** Open the database in read-write mode if possible, or read-only mode if the database is write protected. Attempts to create the database if it doesn't exist. */
	ReadWriteCreate,
};

/**
 * Wrapper around an SQLite database.
 * @see sqlite3.
 */
class FSQLiteDatabase
{
public:
	/** Construction/Destruction */
	UE_API FSQLiteDatabase();
	UE_API ~FSQLiteDatabase();

	/** Non-copyable */
	FSQLiteDatabase(const FSQLiteDatabase&) = delete;
	FSQLiteDatabase& operator=(const FSQLiteDatabase&) = delete;

	/** Movable */
	UE_API FSQLiteDatabase(FSQLiteDatabase&& Other);
	UE_API FSQLiteDatabase& operator=(FSQLiteDatabase&& Other);

	/**
	 * Is this a valid SQLite database? (ie, has been successfully opened).
	 */
	UE_API bool IsValid() const;

	/**
	 * Open (or create) an SQLite database file.
	 */
	UE_API bool Open(const TCHAR* InFilename, const ESQLiteDatabaseOpenMode InOpenMode = ESQLiteDatabaseOpenMode::ReadWriteCreate);

	/**
	 * Close an open SQLite database file.
	 */
	UE_API bool Close();

	/**
	 * Get the filename of the currently open database, or an empty string.
	 * @note The returned filename will be an absolute pathname.
	 */
	UE_API FString GetFilename() const;

	/**
	 * Get the application ID set in the database header.
	 * @note A list of assigned application IDs can be seen by consulting the magic.txt file in the SQLite source repository.
	 * @return true if the get was a success.
	 */
	UE_API bool GetApplicationId(int32& OutApplicationId) const;

	/**
	 * Set the application ID in the database header.
	 * @note A list of assigned application IDs can be seen by consulting the magic.txt file in the SQLite source repository.
	 * @return true if the set was a success.
	 */
	UE_API bool SetApplicationId(const int32 InApplicationId);

	/**
	 * Get the user version set in the database header.
	 * @return true if the get was a success.
	 */
	UE_API bool GetUserVersion(int32& OutUserVersion) const;

	/**
	 * Set the user version in the database header.
	 * @return true if the set was a success.
	 */
	UE_API bool SetUserVersion(const int32 InUserVersion);

	/**
	 * Execute a statement that requires no result state.
	 * @note For statements that require a result, or that you wish to reuse repeatedly (including using bindings), you should consider using FSQLitePreparedStatement.
	 * @return true if the execution was a success.
	 */
	UE_API bool Execute(const TCHAR* InStatement);

	/**
	 * Execute a statement and enumerate the result state.
	 * @note For statements that require a result, or that you wish to reuse repeatedly (including using bindings), you should consider using FSQLitePreparedStatement.
	 * @return The number of rows enumerated (which may be less than the number of rows returned if ESQLitePreparedStatementExecuteRowResult::Stop is returned during enumeration), or INDEX_NONE if an error occurred (including returning ESQLitePreparedStatementExecuteRowResult::Error during enumeration).
	 */
	UE_API int64 Execute(const TCHAR* InStatement, TFunctionRef<ESQLitePreparedStatementExecuteRowResult(const FSQLitePreparedStatement&)> InCallback);

	/**
	 * Prepare a statement for manual processing.
	 * @note This is the same as using the FSQLitePreparedStatement constructor, but won't assert if the current database is invalid (not open).
	 * @return A prepared statement object (check IsValid on the result).
	 */
	UE_API FSQLitePreparedStatement PrepareStatement(const TCHAR* InStatement, const ESQLitePreparedStatementFlags InFlags = ESQLitePreparedStatementFlags::None);

	/**
	 * Prepare a statement defined by SQLITE_PREPARED_STATEMENT for manual processing.
	 * @note This is the same as using the T constructor, but won't assert if the current database is invalid (not open).
	 * @return A prepared statement object (check IsValid on the result).
	 */
	template <typename T>
	T PrepareStatement(const ESQLitePreparedStatementFlags InFlags = ESQLitePreparedStatementFlags::None)
	{
		return Database
			? T(*this, InFlags)
			: T();
	}

	/**
	 * Get the error string associated with the given result code, if any.
	 */
	static UE_API FString GetErrorForCode(const int32 ResultCode);

	/**
	 * Get the last error reported by this database.
	 */
	UE_API FString GetLastError() const;

	/**
	 * Get the rowid of the last successful INSERT statement on any table in this database.
	 * @see sqlite3_last_insert_rowid
	 */
	UE_API int64 GetLastInsertRowId() const;

	/** Performs a quick check on the integrity of the database, returns true if everything is ok. */
	UE_API bool PerformQuickIntegrityCheck() const;

private:
	friend class FSQLitePreparedStatement;

	/** Internal SQLite database handle */
	struct sqlite3* Database;
	
	/** Full original path for logging/profiling */
	FString OriginalPath;

	/** Short name for logging/profiling */
	FString ShortName;
};

#undef UE_API
