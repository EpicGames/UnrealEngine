// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreTypes.h"
#include "SQLiteTypes.h"
#include "Misc/Guid.h"
#include "Misc/DateTime.h"
#include "Misc/EnumClassFlags.h"
#include "UObject/NameTypes.h"
#include "Containers/ArrayView.h"
#include "Containers/UnrealString.h"
#include "Internationalization/Text.h"
#include "Templates/EnableIf.h"
#include "Templates/IsEnumClass.h"
#include "Delegates/IntegerSequence.h"

#define UE_API SQLITECORE_API

class FSQLiteDatabase;

/**
 * Flags used when creating a prepared statement.
 */
enum class ESQLitePreparedStatementFlags : uint8
{
	/** No special flags. */
	None = 0,

	/** Hints that this prepared statement will be retained for a long period and reused many times. */
	Persistent = 1<<0,
};
ENUM_CLASS_FLAGS(ESQLitePreparedStatementFlags);

/**
 * Result codes returned from stepping an SQLite prepared statement.
 */
enum class ESQLitePreparedStatementStepResult : uint8
{
	/** The step was unsuccessful and enumeration should be aborted. */
	Error,

	/** The step was unsuccessful as the required locks could not be acquired. If the statement was outside a transaction (or committing a pending transaction) then you can retry it, otherwise enumeration should be aborted and you should rollback any pending transaction. */
	Busy,

	/** The step was successful and we're on a database row. */
	Row,

	/** The step was successful, but we've reached the end of the rows and enumeration should be aborted. */
	Done,
};

/**
 * Result codes used while executing rows in an SQLite prepared statement.
 */
enum class ESQLitePreparedStatementExecuteRowResult : uint8
{
	/** Continue execution to the next row (if available). */
	Continue,

	/** Stop execution, but do not report an error. */
	Stop,

	/** Stop execution, and report an error. */
	Error,
};

/**
 * Wrapper around an SQLite prepared statement.
 * @see sqlite3_stmt.
 */
class FSQLitePreparedStatement
{
public:
	/** Construction/Destruction */
	UE_API FSQLitePreparedStatement();
	UE_API FSQLitePreparedStatement(FSQLiteDatabase& InDatabase, const TCHAR* InStatement, const ESQLitePreparedStatementFlags InFlags = ESQLitePreparedStatementFlags::None);
	UE_API ~FSQLitePreparedStatement();

	/** Non-copyable */
	FSQLitePreparedStatement(const FSQLitePreparedStatement&) = delete;
	FSQLitePreparedStatement& operator=(const FSQLitePreparedStatement&) = delete;

	/** Movable */
	UE_API FSQLitePreparedStatement(FSQLitePreparedStatement&& Other);
	UE_API FSQLitePreparedStatement& operator=(FSQLitePreparedStatement&& Other);

	/**
	 * Is this a valid SQLite prepared statement? (ie, has been successfully compiled).
	 */
	UE_API bool IsValid() const;

	/**
	 * Is this SQLite prepared statement active? ("busy" in SQLite parlance).
	 * @return true for statements that have been partially stepped and haven't been reset.
	 */
	UE_API bool IsActive() const;

	/**
	 * Is this SQLite prepared statement read-only? (ie, will it only read from the database contents?).
	 */
	UE_API bool IsReadOnly() const;

	/**
	 * Create a new SQLite prepared statement.
	 */
	UE_API bool Create(FSQLiteDatabase& InDatabase, const TCHAR* InStatement, const ESQLitePreparedStatementFlags InFlags = ESQLitePreparedStatementFlags::None);

	/**
	 * Destroy the existing SQLite prepared statement.
	 */
	UE_API bool Destroy();

	/**
	 * Reset the SQLite prepared statement so that it can be used again.
	 * @note This doesn't remove any existing bindings (see ClearBindings).
	 */
	UE_API void Reset();

	/**
	 * Clear any bindings that have been applied to this prepared statement.
	 */
	UE_API void ClearBindings();

	/**
	 * Get the index of a given binding from its name.
	 * @return The binding index, or 0 if it could not be found.
	 */
	UE_API int32 GetBindingIndexByName(const TCHAR* InBindingName) const;

	/**
	 * Set the given integer binding from its name or index.
	 */
	UE_API bool SetBindingValueByName(const TCHAR* InBindingName, const int8 InValue);
	UE_API bool SetBindingValueByIndex(const int32 InBindingIndex, const int8 InValue);
	UE_API bool SetBindingValueByName(const TCHAR* InBindingName, const uint8 InValue);
	UE_API bool SetBindingValueByIndex(const int32 InBindingIndex, const uint8 InValue);
	UE_API bool SetBindingValueByName(const TCHAR* InBindingName, const int16 InValue);
	UE_API bool SetBindingValueByIndex(const int32 InBindingIndex, const int16 InValue);
	UE_API bool SetBindingValueByName(const TCHAR* InBindingName, const uint16 InValue);
	UE_API bool SetBindingValueByIndex(const int32 InBindingIndex, const uint16 InValue);
	UE_API bool SetBindingValueByName(const TCHAR* InBindingName, const int32 InValue);
	UE_API bool SetBindingValueByIndex(const int32 InBindingIndex, const int32 InValue);
	UE_API bool SetBindingValueByName(const TCHAR* InBindingName, const uint32 InValue);
	UE_API bool SetBindingValueByIndex(const int32 InBindingIndex, const uint32 InValue);
	UE_API bool SetBindingValueByName(const TCHAR* InBindingName, const int64 InValue);
	UE_API bool SetBindingValueByIndex(const int32 InBindingIndex, const int64 InValue);
	UE_API bool SetBindingValueByName(const TCHAR* InBindingName, const uint64 InValue);
	UE_API bool SetBindingValueByIndex(const int32 InBindingIndex, const uint64 InValue);
	UE_API bool SetBindingValueByName(const TCHAR* InBindingName, const FDateTime InValue);
	UE_API bool SetBindingValueByIndex(const int32 InBindingIndex, const FDateTime InValue);
	template <typename EnumType, typename = typename TEnableIf<TIsEnumClass<EnumType>::Value>::Type>
	bool SetBindingValueByName(const TCHAR* InBindingName, const EnumType InValue)
	{
		return SetBindingValueByIndex(GetBindingIndexByName(InBindingName), InValue);
	}
	template <typename EnumType, typename = typename TEnableIf<TIsEnumClass<EnumType>::Value>::Type>
	bool SetBindingValueByIndex(const int32 InBindingIndex, const EnumType InValue)
	{
		return SetBindingValueByIndex(InBindingIndex, (__underlying_type(EnumType))InValue);
	}

	/**
	 * Set the given float binding from its name or index.
	 */
	UE_API bool SetBindingValueByName(const TCHAR* InBindingName, const float InValue);
	UE_API bool SetBindingValueByIndex(const int32 InBindingIndex, const float InValue);
	UE_API bool SetBindingValueByName(const TCHAR* InBindingName, const double InValue);
	UE_API bool SetBindingValueByIndex(const int32 InBindingIndex, const double InValue);

	/**
	 * Set the given string binding from its name or index.
	 */
	UE_API bool SetBindingValueByName(const TCHAR* InBindingName, const TCHAR* InValue);
	UE_API bool SetBindingValueByIndex(const int32 InBindingIndex, const TCHAR* InValue);
	UE_API bool SetBindingValueByName(const TCHAR* InBindingName, const FString& InValue);
	UE_API bool SetBindingValueByIndex(const int32 InBindingIndex, const FString& InValue);
	UE_API bool SetBindingValueByName(const TCHAR* InBindingName, const FName InValue);
	UE_API bool SetBindingValueByIndex(const int32 InBindingIndex, const FName InValue);
	UE_API bool SetBindingValueByName(const TCHAR* InBindingName, const FText& InValue);
	UE_API bool SetBindingValueByIndex(const int32 InBindingIndex, const FText& InValue);

	/**
	 * Set the given blob binding from its name or index.
	 * @note If bCopy is set to false, then you must ensure the memory bound remains valid for the duration that the prepared statement is using it.
	 */
	UE_API bool SetBindingValueByName(const TCHAR* InBindingName, TArrayView<const uint8> InBlobData, const bool bCopy = true);
	UE_API bool SetBindingValueByIndex(const int32 InBindingIndex, TArrayView<const uint8> InBlobData, const bool bCopy = true);
	UE_API bool SetBindingValueByName(const TCHAR* InBindingName, const void* InBlobData, const int32 InBlobDataSizeBytes, const bool bCopy = true);
	UE_API bool SetBindingValueByIndex(const int32 InBindingIndex, const void* InBlobData, const int32 InBlobDataSizeBytes, const bool bCopy = true);
	UE_API bool SetBindingValueByName(const TCHAR* InBindingName, const FGuid& InValue);
	UE_API bool SetBindingValueByIndex(const int32 InBindingIndex, const FGuid& InValue);

	/**
	 * Set the given null binding from its name or index.
	 */
	UE_API bool SetBindingValueByName(const TCHAR* InBindingName);
	UE_API bool SetBindingValueByIndex(const int32 InBindingIndex);

	/**
	 * Execute a statement that requires no result state.
	 * @note The statement must not be active, and any required bindings must have been set before calling this function (this function will not modify bindings).
	 * @return true if the execution was a success.
	 */
	UE_API bool Execute();

	/**
	 * Execute a statement and enumerate the result state.
	 * @note The statement must not be active, and any required bindings must have been set before calling this function (this function will not modify bindings).
	 * @return The number of rows enumerated (which may be less than the number of rows returned if ESQLitePreparedStatementExecuteRowResult::Stop is returned during enumeration), or INDEX_NONE if an error occurred (including returning ESQLitePreparedStatementExecuteRowResult::Error during enumeration).
	 */
	UE_API int64 Execute(TFunctionRef<ESQLitePreparedStatementExecuteRowResult(const FSQLitePreparedStatement&)> InCallback);

	/**
	 * Step the SQLite prepared statement to try and move on to the next result from the statement.
	 * @param OutErrorCode Optional value to store the low-level error code of the step (@see FSQLiteDatabase::GetErrorForCode).
	 * @note See FSQLiteDatabase::Execute for a simple example of stepping a statement.
	 */
	UE_API ESQLitePreparedStatementStepResult Step(int32* OutErrorCode = nullptr);

	/**
	 * Get the index of a column from its name.
	 * @note It's better to look-up a column index once rather than look it up for each access request.
	 * @return The column index, or INDEX_NONE if it couldn't be found.
	 */
	UE_API int32 GetColumnIndexByName(const TCHAR* InColumnName) const;

	/**
	 * Get the integer value of a column from its name or index.
	 */
	UE_API bool GetColumnValueByName(const TCHAR* InColumnName, int8& OutValue) const;
	UE_API bool GetColumnValueByIndex(const int32 InColumnIndex, int8& OutValue) const;
	UE_API bool GetColumnValueByName(const TCHAR* InColumnName, uint8& OutValue) const;
	UE_API bool GetColumnValueByIndex(const int32 InColumnIndex, uint8& OutValue) const;
	UE_API bool GetColumnValueByName(const TCHAR* InColumnName, int16& OutValue) const;
	UE_API bool GetColumnValueByIndex(const int32 InColumnIndex, int16& OutValue) const;
	UE_API bool GetColumnValueByName(const TCHAR* InColumnName, uint16& OutValue) const;
	UE_API bool GetColumnValueByIndex(const int32 InColumnIndex, uint16& OutValue) const;
	UE_API bool GetColumnValueByName(const TCHAR* InColumnName, int32& OutValue) const;
	UE_API bool GetColumnValueByIndex(const int32 InColumnIndex, int32& OutValue) const;
	UE_API bool GetColumnValueByName(const TCHAR* InColumnName, uint32& OutValue) const;
	UE_API bool GetColumnValueByIndex(const int32 InColumnIndex, uint32& OutValue) const;
	UE_API bool GetColumnValueByName(const TCHAR* InColumnName, int64& OutValue) const;
	UE_API bool GetColumnValueByIndex(const int32 InColumnIndex, int64& OutValue) const;
	UE_API bool GetColumnValueByName(const TCHAR* InColumnName, uint64& OutValue) const;
	UE_API bool GetColumnValueByIndex(const int32 InColumnIndex, uint64& OutValue) const;
	UE_API bool GetColumnValueByName(const TCHAR* InColumnName, FDateTime& OutValue) const;
	UE_API bool GetColumnValueByIndex(const int32 InColumnIndex, FDateTime& OutValue) const;
	template <typename EnumType, typename = typename TEnableIf<TIsEnumClass<EnumType>::Value>::Type>
	bool GetColumnValueByName(const TCHAR* InColumnName, EnumType& InValue) const
	{
		return GetColumnValueByIndex(GetColumnIndexByName(InColumnName), InValue);
	}
	template <typename EnumType, typename = typename TEnableIf<TIsEnumClass<EnumType>::Value>::Type>
	bool GetColumnValueByIndex(const int32 InColumnIndex, EnumType& InValue) const
	{
		return GetColumnValueByIndex(InColumnIndex, (__underlying_type(EnumType)&)InValue);
	}

	/**
	 * Get the float value of a column from its name or index.
	 */
	UE_API bool GetColumnValueByName(const TCHAR* InColumnName, float& OutValue) const;
	UE_API bool GetColumnValueByIndex(const int32 InColumnIndex, float& OutValue) const;
	UE_API bool GetColumnValueByName(const TCHAR* InColumnName, double& OutValue) const;
	UE_API bool GetColumnValueByIndex(const int32 InColumnIndex, double& OutValue) const;

	/**
	 * Get the string value of a column from its name or index.
	 */
	UE_API bool GetColumnValueByName(const TCHAR* InColumnName, FString& OutValue) const;
	UE_API bool GetColumnValueByIndex(const int32 InColumnIndex, FString& OutValue) const;
	UE_API bool GetColumnValueByName(const TCHAR* InColumnName, FName& OutValue) const;
	UE_API bool GetColumnValueByIndex(const int32 InColumnIndex, FName& OutValue) const;
	UE_API bool GetColumnValueByName(const TCHAR* InColumnName, FText& OutValue) const;
	UE_API bool GetColumnValueByIndex(const int32 InColumnIndex, FText& OutValue) const;

	/**
	 * Get the blob value of a column from its name or index.
	 */
	UE_API bool GetColumnValueByName(const TCHAR* InColumnName, TArray<uint8>& OutValue) const;
	UE_API bool GetColumnValueByIndex(const int32 InColumnIndex, TArray<uint8>& OutValue) const;
	UE_API bool GetColumnValueByName(const TCHAR* InColumnName, FGuid& OutValue) const;
	UE_API bool GetColumnValueByIndex(const int32 InColumnIndex, FGuid& OutValue) const;

	/**
	 * Get the type of a column from its name or index.
	 * @note Column types in SQLite are somewhat arbitrary are not enforced, nor need to be consistent between the same column in different rows.
	 */
	UE_API bool GetColumnTypeByName(const TCHAR* InColumnName, ESQLiteColumnType& OutColumnType) const;
	UE_API bool GetColumnTypeByIndex(const int32 InColumnIndex, ESQLiteColumnType& OutColumnType) const;

	/**
	 * Get the column names affected by this statement.
	 */
	UE_API const TArray<FString>& GetColumnNames() const;

private:
	/** Attempt to cache the column names, if required and possible */
	UE_API void CacheColumnNames() const;

	/** Check whether the given column index is within the range of available columns */
	UE_API bool IsValidColumnIndex(const int32 InColumnIndex) const;

	/**
	 * Set the given integer binding from its name or index.
	 */
	template <typename T>
	bool SetBindingValueByName_Integer(const TCHAR* InBindingName, const T InValue);
	template <typename T>
	bool SetBindingValueByIndex_Integer(const int32 InBindingIndex, const T InValue);

	/**
	 * Set the given float binding from its name or index.
	 */
	template <typename T>
	bool SetBindingValueByName_Real(const TCHAR* InBindingName, const T InValue);
	template <typename T>
	bool SetBindingValueByIndex_Real(const int32 InBindingIndex, const T InValue);

	/**
	 * Get the integer value of a column from its name or index.
	 */
	template <typename T>
	bool GetColumnValueByName_Integer(const TCHAR* InColumnName, T& OutValue) const;
	template <typename T>
	bool GetColumnValueByIndex_Integer(const int32 InColumnIndex, T& OutValue) const;

	/**
	 * Get the float value of a column from its name or index.
	 */
	template <typename T>
	bool GetColumnValueByName_Real(const TCHAR* InColumnName, T& OutValue) const;
	template <typename T>
	bool GetColumnValueByIndex_Real(const int32 InColumnIndex, T& OutValue) const;

	/** Internal SQLite prepared statement handle */
	struct sqlite3_stmt* Statement;

	/** Cached array of column names (generated on-demand when needed by the API) */
	mutable TArray<FString> CachedColumnNames;
};

/** Macro wrapper for the columns and bindings mixin template types, so that they can be used as an argument to other macros */
#define SQLITE_PREPARED_STATEMENT_COLUMNS(...)	SQLitePreparedStatementImpl::TColumns<__VA_ARGS__>
#define SQLITE_PREPARED_STATEMENT_BINDINGS(...)	SQLitePreparedStatementImpl::TBindings<__VA_ARGS__>

/** Macro wrapper that takes the columns and bindings as a variable args list, so that it can deal with the expanded version of the macros above containing commas which it treats as extra arguments to the macro */
#define SQLITE_PREPARED_STATEMENT_EXPANDED_IMPL(TYPE, STATEMENT, ...)		\
	struct _##TYPE##_StatementStr_Provider { static const TCHAR* GetStatement() { return TEXT(STATEMENT); } };		\
	typedef SQLitePreparedStatementImpl::TPreparedStatement<_##TYPE##_StatementStr_Provider, ##__VA_ARGS__> TYPE

/** Macro to define a type-safe prepared statement type that may have optional columns and bindings */
#define SQLITE_PREPARED_STATEMENT(TYPE, STATEMENT, COLUMNS, BINDINGS)		\
	SQLITE_PREPARED_STATEMENT_EXPANDED_IMPL(TYPE, STATEMENT, COLUMNS, BINDINGS)

/** Macro to define a type-safe prepared statement type with only columns */
#define SQLITE_PREPARED_STATEMENT_COLUMNS_ONLY(TYPE, STATEMENT, COLUMNS)	\
	SQLITE_PREPARED_STATEMENT_EXPANDED_IMPL(TYPE, STATEMENT, COLUMNS, SQLITE_PREPARED_STATEMENT_BINDINGS())

/** Macro to define a type-safe prepared statement type with only bindings */
#define SQLITE_PREPARED_STATEMENT_BINDINGS_ONLY(TYPE, STATEMENT, BINDINGS)	\
	SQLITE_PREPARED_STATEMENT_EXPANDED_IMPL(TYPE, STATEMENT, SQLITE_PREPARED_STATEMENT_COLUMNS(), BINDINGS)

/** Macro to define a type-safe prepared statement type with neither columns or bindings */
#define SQLITE_PREPARED_STATEMENT_SIMPLE(TYPE, STATEMENT)					\
	SQLITE_PREPARED_STATEMENT_EXPANDED_IMPL(TYPE, STATEMENT, SQLITE_PREPARED_STATEMENT_COLUMNS(), SQLITE_PREPARED_STATEMENT_BINDINGS())

namespace SQLitePreparedStatementImpl
{

/**
 * Type-safe packing of multiple column types.
 * For use with SQLITE_PREPARED_STATEMENT_COLUMNS.
 */
template <typename... Columns>
struct TColumns;

/**
 * Type-safe packing of multiple binding types.
 * For use with SQLITE_PREPARED_STATEMENT_BINDINGS.
 */
template <typename... Bindings>
struct TBindings;

/**
 * Generic version of TPreparedStatement, specialized below.
 */
template <typename StatementProvider, typename Columns, typename Bindings>
class TPreparedStatement;

/**
 * Creates a type-safe SQLite prepared statement.
 * For use with SQLITE_PREPARED_STATEMENT.
 */
template <typename StatementProvider, typename... Columns, typename... Bindings>
class TPreparedStatement<StatementProvider, TColumns<Columns...>, TBindings<Bindings...>> : public FSQLitePreparedStatement
{
public:
	TPreparedStatement() = default;
	TPreparedStatement(FSQLiteDatabase& InDatabase, const ESQLitePreparedStatementFlags InFlags)
		: FSQLitePreparedStatement(InDatabase, StatementProvider::GetStatement(), InFlags)
	{
	}

	/**
	 * Create a new SQLite prepared statement.
	 */
	bool Create(FSQLiteDatabase& InDatabase, const ESQLitePreparedStatementFlags InFlags = ESQLitePreparedStatementFlags::None)
	{
		return FSQLitePreparedStatement::Create(InDatabase, StatementProvider::GetStatement(), InFlags);
	}

	/**
	 * Execute a statement that requires no result state.
	 * @note The statement must not be active, and any required bindings must have been set before calling this function (this function will not modify bindings).
	 * @return true if the execution was a success.
	 */
	bool Execute()
	{
		return FSQLitePreparedStatement::Execute();
	}

	/**
	 * Set the value of all bindings, and execute a statement that requires no result state.
	 * @note The statement must not be active.
	 * @return true if the execution was a success.
	 */
	bool BindAndExecute(const Bindings&... BindingArgs)
	{
		bool bResult = false;
		if (SetBindingValues(BindingArgs...))
		{
			bResult = Execute();
			ClearBindings();
		}
		return bResult;
	}

	/**
	 * Execute a statement and enumerate the result state.
	 * @note The statement must not be active, and any required bindings must have been set before calling this function (this function will not modify bindings).
	 * @return The number of rows enumerated (which may be less than the number of rows returned if ESQLitePreparedStatementExecuteRowResult::Stop is returned during enumeration), or INDEX_NONE if an error occurred (including returning ESQLitePreparedStatementExecuteRowResult::Error during enumeration).
	 */
	int64 Execute(TFunctionRef<ESQLitePreparedStatementExecuteRowResult(const TPreparedStatement&)> InCallback)
	{
		return FSQLitePreparedStatement::Execute([this, &InCallback](const FSQLitePreparedStatement& InStatement)
		{
			check(this == &InStatement);
			return InCallback(*this);
		});
	}

	/**
	 * Set the value of all bindings, and execute a statement and enumerate the result state.
	 * @note The statement must not be active.
	 * @return The number of rows enumerated (which may be less than the number of rows returned if ESQLitePreparedStatementExecuteRowResult::Stop is returned during enumeration), or INDEX_NONE if an error occurred (including returning ESQLitePreparedStatementExecuteRowResult::Error during enumeration).
	 */
	int64 BindAndExecute(const Bindings&... BindingArgs, TFunctionRef<ESQLitePreparedStatementExecuteRowResult(const TPreparedStatement&)> InCallback)
	{
		int64 Result = INDEX_NONE;
		if (SetBindingValues(BindingArgs...))
		{
			Result = Execute(InCallback);
			ClearBindings();
		}
		return Result;
	}

	/**
	 * Execute a statement that returns a single result.
	 * @note The statement must not be active, and any required bindings must have been set before calling this function (this function will not modify bindings).
	 * @return true if the execution was a success and a single result was returned.
	 */
	bool ExecuteSingle(Columns&... ColumnArgs)
	{
		return Execute([&](const TPreparedStatement& InStatement)
		{
			return InStatement.GetColumnValues(ColumnArgs...) ? ESQLitePreparedStatementExecuteRowResult::Continue : ESQLitePreparedStatementExecuteRowResult::Error;
		}) == 1;
	}

	/**
	 * Set the value of all bindings, and execute a statement that returns a single result.
	 * @note The statement must not be active.
	 * @return true if the execution was a success and a single result was returned.
	 */
	bool BindAndExecuteSingle(const Bindings&... BindingArgs, Columns&... ColumnArgs)
	{
		bool bResult = false;
		if (SetBindingValues(BindingArgs...))
		{
			bResult = ExecuteSingle(ColumnArgs...);
			ClearBindings();
		}
		return bResult;
	}

	/**
	 * Set the value of all bindings.
	 */
	bool SetBindingValues(const Bindings&... BindingArgs)
	{
		return SetBindingValuesImpl(TMakeIntegerSequence<int32, sizeof...(Bindings)>(), BindingArgs...);
	}

	/**
	 * Get the values of all columns.
	 */
	bool GetColumnValues(Columns&... ColumnArgs) const
	{
		return GetColumnValuesImpl(TMakeIntegerSequence<int32, sizeof...(Columns)>(), ColumnArgs...);
	}

private:
	/**
	 * Set the value of all bindings.
	 */
	template <int32... BindingIndices>
	bool SetBindingValuesImpl(TIntegerSequence<int32, BindingIndices...> IntSeq, const Bindings&... BindingArgs)
	{
		bool bResult = true;
		bool DummyUnpackArray[] = { bResult &= SetBindingValueByIndex(BindingIndices + 1, BindingArgs)..., false }; // Dummy false entry to deal with zero-size case
		(void)DummyUnpackArray; // Force use the dummy variable
		return bResult;
	}

	/**
	 * Get the values of all columns.
	 */
	template <int32... ColumnIndices>
	bool GetColumnValuesImpl(TIntegerSequence<int32, ColumnIndices...> IntSeq, Columns&... ColumnArgs) const
	{
		bool bResult = true;
		bool DummyUnpackArray[] = { bResult &= GetColumnValueByIndex(ColumnIndices, ColumnArgs)..., false }; // Dummy false entry to deal with zero-size case
		(void)DummyUnpackArray; // Force use the dummy variable
		return bResult;
	}
};

} // namespace SQLitePreparedStatementImpl

#undef UE_API
