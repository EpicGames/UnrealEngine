// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreTypes.h"

/**
 * Data types for columns in an SQLite table.
 */
enum class ESQLiteColumnType : uint8
{
	/** 64-bit signed integer */
	Integer,

	/** 64-bit IEEE floating point number */
	Float,

	/** String */
	String,

	/** BLOB */
	Blob,

	/** NULL */
	Null,
};
