// Copyright Epic Games, Inc. All Rights Reserved.

using System.IO;

namespace UnrealBuildTool.Rules
{
	public class SQLiteCore : ModuleRules
	{
		public SQLiteCore(ReadOnlyTargetRules Target) : base(Target)
		{
			IWYUSupport = IWYUSupport.None;

			PublicDependencyModuleNames.AddRange(
				new string[] {
					"Core",
					"TraceLog"
				}
			);

			// Allow direct access to the SQLite API from outside this module
			// Note: There are additional macros defined inside SQLiteEmbedded.c to allow this API export to work
			PublicDefinitions.Add("SQLITE_API=SQLITECORE_API");

			// We call sqlite3_initialize ourselves during module init
			PrivateDefinitions.Add("SQLITE_OMIT_AUTOINIT");

			// Use the math.h version of isnan rather than the SQLite version to avoid a -ffast-math error
			PrivateDefinitions.Add("SQLITE_HAVE_ISNAN=1");

			// Enable SQLite debug checks?
			//PrivateDefinitions.Add("SQLITE_DEBUG");

			// Enable FTS
			PrivateDefinitions.Add("SQLITE_ENABLE_FTS4");
			PrivateDefinitions.Add("SQLITE_ENABLE_FTS5");

			// Enable Deserialization and RTree Subsystem
			PrivateDefinitions.Add("SQLITE_ENABLE_DESERIALIZE");
			PrivateDefinitions.Add("SQLITE_ENABLE_RTREE");

			// Enable Json extension
			PrivateDefinitions.Add("SQLITE_ENABLE_JSON1");

			PublicIncludePaths.Add(Path.Combine(ModuleDirectory, "..", "ThirdParty"));
			// Use ICU with SQLite if it's available
			if (Target.bCompileICU)
			{
				PrivateDefinitions.Add("SQLITE_ENABLE_ICU");
				AddEngineThirdPartyPrivateStaticDependencies(Target, "ICU");
			}

			// Should we use the Unreal HAL rather than the SQLite platform implementations?
			if (Target.bCompileCustomSQLitePlatform)
			{
				// Note: The Unreal HAL doesn't provide an implementation of shared memory (as not all platforms implement it),
				// nor does it provide an implementation of granular file locks. These two things affect the concurrency of an
				// SQLite database as only one FSQLiteDatabase can have the file open at the same time.
				PrivateDefinitions.Add("SQLITE_OS_OTHER=1");			// We are a custom OS
				PrivateDefinitions.Add("SQLITE_ZERO_MALLOC");			// We provide our own malloc implementation
				PrivateDefinitions.Add("SQLITE_MUTEX_NOOP");			// We provide our own mutex implementation
				PrivateDefinitions.Add("SQLITE_OMIT_LOAD_EXTENSION");	// We disable extension loading
			}
		}
	}
}
